// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MediaUtils : ModuleRules
	{
		public MediaUtils(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"ImageWrapper",
					"Media",
					"SignalProcessing"
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"ImageWriteQueue",
				});
		}
	}
}
