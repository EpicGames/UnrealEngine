// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppMediaTimeSource.h"
#include "MediaUtilsPrivate.h"

#include "Misc/App.h"
#include "Misc/Timespan.h"

/** Declares a log category for this module. */
DECLARE_LOG_CATEGORY_CLASS(LogMediaTimeSource, Log, All);

/* IMediaTimeSource interface 
 *****************************************************************************/

FTimespan FAppMediaTimeSource::GetTimecode()
{
	const double CurrentTime = FApp::GetGameTime();
	const FTimespan Timecode = FTimespan::FromSeconds(CurrentTime);

	UE_LOGF(LogMediaTimeSource, VeryVerbose, "AppMediaTimeSource: Time %.10f, Delta %.10f, Timecode %ls", CurrentTime, FApp::GetDeltaTime(), *Timecode.ToString(TEXT("%h:%m:%s.%t")));

	return Timecode;
}
