// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"


class FLinearEase
{
public:
	FLinearEase()
		: StartValue(0.0f)
		, CurrentValue(0.0f)
		, DeltaValue(0.0f)
		, SampleRate(44100.0f)
		, DurationTicks(0)
		, DefaultDurationTicks(0)
		, CurrentTick(0)
	{ }

	~FLinearEase() = default;

	bool IsDone() const
	{
		return CurrentTick >= DurationTicks;
	}

	void Init(float InSampleRate)
	{
		SampleRate = InSampleRate;
	}

	void SetValueRange(const float Start, const float End, const float InTimeSec)
	{
		StartValue = Start;
		CurrentValue = Start;
		SetValue(End, InTimeSec);
	}

	float GetValue()
	{
		if (IsDone())
		{
			return CurrentValue;
		}

		CurrentValue = DeltaValue * (float)CurrentTick / DurationTicks + StartValue;

		++CurrentTick;
		return CurrentValue;
	}

	// Updates the target value without changing the duration or tick data.
	// Sets the state as if the new value was the target value all along
	void SetValueInterrupt(const float InValue)
	{
		if (IsDone())
		{
			CurrentValue = InValue;
		}
		else
		{
			DurationTicks = DurationTicks - CurrentTick;
			CurrentTick = 0;
			DeltaValue = InValue - CurrentValue;
			StartValue = CurrentValue;
		}
	}

	void SetValue(const float InValue, float InTimeSec = 0.0f)
	{
		DurationTicks = (int32)(SampleRate * InTimeSec);
		CurrentTick = 0;

		if (DurationTicks == 0)
		{
			CurrentValue = InValue;
		}
		else
		{
			DeltaValue = InValue - CurrentValue;
			StartValue = CurrentValue;
		}
	}

private:

	float StartValue;
	float CurrentValue;
	float DeltaValue;
	float SampleRate;
	int32 DurationTicks;
	int32 DefaultDurationTicks;
	int32 CurrentTick;
};
