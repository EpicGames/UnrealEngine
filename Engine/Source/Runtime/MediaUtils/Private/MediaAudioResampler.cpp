// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaAudioResampler.h"
#include "MediaUtilsPrivate.h"
#include "DSP/FloatArrayMath.h"
#include "Algo/Reverse.h"
#include "IMediaAudioSample.h"


/* Local helpers
 *****************************************************************************/

namespace MediaAudioResampler
{
	// Rows: output speaker configuration
	// Cols: input source channels

	// Conversion to Mono -----------------------------------------------------

	static const float Matrix_1_1[] = {
		// Mono
		1.0f,		// Mono
	};

	static const float Matrix_2_1[] = {
		// Left			Right
		0.5f,			0.5f,		// Mono
	};

	static const float Matrix_3_1[] = {
		// Left			Right		Center
		0.707f,			0.707f,		0.5f,		// Mono
	};

	static const float Matrix_4_1[] = {
		// FrontLeft	FrontRight	SideLeft	SideRight
		0.707f,			0.707f,		0.5f,		0.5f,		// Mono
	};

	static const float Matrix_5_1[] = {
		// FrontLeft	FrontRight	Center		SideLeft	SideRight
		0.707f,			0.707f,		0.5f,		0.5f,		0.5f,		// Mono
	};

	static const float Matrix_6_1[] = {
		// FrontLeft	FrontRight	Center		LowFreq		SideLeft	SideRight
		0.707f,			0.707f,		0.5f,		0.0f,		0.5f,		0.5f,		// Mono
	};

	static const float Matrix_7_1[] = {
		// FrontLeft	FrontRight	BackLeft	LFE			BackRight	SideLeft	SideRight
		0.707f,			0.707f,		0.5f,		0.0f,		0.5f,		0.5f,		0.5f,		// Mono
	};

	static const float Matrix_8_1[] = {
		// FrontLeft	FrontRight	Center		LowFreq		SideLeft	SideRight	BackLeft	BackRight
		0.707f,			0.707f,		1.0f,		0.0f,		0.5f,		0.5f,		0.5f,		0.5f,		// Mono
	};

	static const float* ToMono[8] = {
		Matrix_1_1,
		Matrix_2_1,
		Matrix_3_1,
		Matrix_4_1,
		Matrix_5_1,
		Matrix_6_1,
		Matrix_7_1,
		Matrix_8_1,
	};

	// Conversion to Stereo ---------------------------------------------------

	static const float Matrix_1_2[] = {
		// Mono
		0.707f,		// Left
		0.707f,		// Right
	};

	static const float Matrix_2_2[] = {
		// Left			Right
		1.0f,			0.0f,		// Left
		0.0f,			1.0f,		// Right
	};

	static const float Matrix_3_2[] = {
		// Left			Right		Center
		1.0f,			0.0f,		0.707f,		// Left
		0.0f,			1.0f,		0.707f,		// Right
	};

	static const float Matrix_4_2[] = {
		// FrontLeft	FrontRight	SideLeft	SideRight
		1.0f,			0.0f,		0.707f,		0.0f,		// Left
		0.0f,			1.0f,		0.0f,		0.707f,		// Right
	};

	static const float Matrix_5_2[] = {
		// FrontLeft	FrontRight	Center		SideLeft	SideRight
		1.0f,			0.0f,		0.707f,		0.707f,		0.0f,		// Left
		0.0f,			1.0f,		0.707f,		0.0f,		0.707f,		// Right
	};

	static const float Matrix_6_2[] = {
		// FrontLeft	FrontRight	Center		LowFreq		SideLeft	SideRight
		1.0f,			0.0f,		0.707f,		0.0f,		0.707f,		0.0f,		// Left
		0.0f,			1.0f,		0.707f,		0.0f,		0.0f,		0.707f,		// Right
	};

	static const float Matrix_7_2[] = {
		// FrontLeft	FrontRight	BackLeft	LFE			BackRight	SideLeft	SideRight
		1.0f,			0.0f,		0.707f,		0.0f,		0.0f,		0.707f,		0.0f,		// Left
		0.0f,			1.0f,		0.0f,		0.0f,		0.707f,		0.0f,		0.707f,		// Right
	};

	static const float Matrix_8_2[] = {
		// FrontLeft	FrontRight	Center		LowFreq		SideLeft	SideRight	BackLeft	BackRight
		1.0f,			0.0f,		0.707f,		0.0f,		0.707f,		0.0f,		0.707f,		0.0f,		// Left
		0.0f,			1.0f,		0.707f,		0.0f,		0.0f,		0.707f,		0.0f,		0.707f,		// Right
	};

	static const float* ToStereo[8] = {
		Matrix_1_2,
		Matrix_2_2,
		Matrix_3_2,
		Matrix_4_2,
		Matrix_5_2,
		Matrix_6_2,
		Matrix_7_2,
		Matrix_8_2,
	};


	// Conversion to 7.1 ------------------------------------------------------

	static const float Matrix_1_8[] = {
		// Mono
		0.0f,		// FrontLeft
		0.0f,		// FrontRight
		1.0f,		// Center
		0.0f,		// LowFrequency
		0.0f,		// SideLeft
		0.0f,		// SideRight
		0.0f,		// BackLeft
		0.0f,		// BackRight
	};

	static const float Matrix_2_8[] = {
		// FrontLeft	FrontRight
		1.0f,			0.0f,			// FrontLeft
		0.0f,			1.0f,			// FrontRight
		0.0f,			0.0f,			// Center
		0.0f,			0.0f,			// LowFrequency
		0.0f,			0.0f,			// SideLeft
		0.0f,			0.0f,			// SideRight
		0.0f,			0.0f,			// BackLeft
		0.0f,			0.0f,			// BackRight
	};

	static const float Matrix_3_8[] = {
		// FrontLeft	FrontRight	Center
		1.0f,			0.0f,		0.0f,	// FrontLeft
		0.0f,			1.0f,		0.0f,	// FrontRight
		0.0f,			0.0f,		1.0f,	// Center
		0.0f,			0.0f,		0.0f,	// LowFrequency
		0.0f,			0.0f,		0.0f,	// SideLeft
		0.0f,			0.0f,		0.0f,	// SideRight
		0.0f,			0.0f,		0.0f,	// BackLeft
		0.0f,			0.0f,		0.0f,	// BackRight
	};

	static const float Matrix_4_8[] = {
		// FrontLeft	FrontRight	SideLeft	SideRight
		1.0f,			0.0f,		0.0f,		0.0f,		// FrontLeft
		0.0f,			1.0f,		0.0f,		0.0f,		// FrontRight
		0.0f,			0.0f,		0.0f,		0.0f,		// Center
		0.0f,			0.0f,		0.0f,		0.0f,		// LowFrequency
		0.0f,			0.0f,		1.0f,		0.0f,		// SideLeft
		0.0f,			0.0f,		0.0f,		1.0f,		// SideRight
		0.0f,			0.0f,		0.0f,		0.0f,		// BackLeft
		0.0f,			0.0f,		0.0f,		0.0f,		// BackRight
	};

	static const float Matrix_5_8[] = {
		// FrontLeft	FrontRight	Center		SideLeft	SideRight
		1.0f,			0.0f,		0.0f,		0.0f,		0.0f,		// FrontLeft
		0.0f,			1.0f,		0.0f,		0.0f,		0.0f,		// FrontRight
		0.0f,			0.0f,		1.0f,		0.0f,		0.0f,		// Center
		0.0f,			0.0f,		0.0f,		0.0f,		0.0f,		// LowFrequency
		0.0f,			0.0f,		0.0f,		1.0f,		0.0f,		// SideLeft
		0.0f,			0.0f,		0.0f,		0.0f,		1.0f,		// SideRight
		0.0f,			0.0f,		0.0f,		0.0f,		0.0f,		// BackLeft
		0.0f,			0.0f,		0.0f,		0.0f,		0.0f,		// BackRight
	};

	static const float Matrix_6_8[] = {
		// FrontLeft	FrontRight	Center		LowFreq		SideLeft	SideRight
		1.0f,			0.0f,		0.0f,		0.0f,		0.0f,		0.0f,		// FrontLeft
		0.0f,			1.0f,		0.0f,		0.0f,		0.0f,		0.0f,		// FrontRight
		0.0f,			0.0f,		1.0f,		0.0f,		0.0f,		0.0f,		// Center
		0.0f,			0.0f,		0.0f,		1.0f,		0.0f,		0.0f,		// LowFrequency
		0.0f,			0.0f,		0.0f,		0.0f,		1.0f,		0.0f,		// SideLeft
		0.0f,			0.0f,		0.0f,		0.0f,		0.0f,		1.0f,		// SideRight
		0.0f,			0.0f,		0.0f,		0.0f,		0.0f,		0.0f,		// BackLeft
		0.0f,			0.0f,		0.0f,		0.0f,		0.0f,		0.0f,		// BackRight
	};

	static const float Matrix_7_8[] = {
		// FrontLeft	FrontRight	BackLeft	LFE			BackRight	SideLeft	SideRight
		1.0f,			0.0f,		0.0f,		0.0f,		0.0f,		0.0f,		0.0f,		// FrontLeft
		0.0f,			1.0f,		0.0f,		0.0f,		0.0f,		0.0f,		0.0f,		// FrontRight
		0.0f,			0.0f,		0.0f,		0.0f,		0.0f,		0.0f,		0.0f,		// Center
		0.0f,			0.0f,		0.0f,		1.0f,		0.0f,		0.0f,		0.0f,		// LowFrequency
		0.0f,			0.0f,		0.0f,		0.0f,		0.0f,		1.0f,		0.0f,		// SideLeft
		0.0f,			0.0f,		0.0f,		0.0f,		0.0f,		0.0f,		1.0f,		// SideRight
		0.0f,			0.0f,		1.0f,		0.0f,		0.0f,		0.0f,		0.0f,		// BackLeft
		0.0f,			0.0f,		0.0f,		0.0f,		1.0f,		0.0f,		0.0f,		// BackRight
	};

	static const float Matrix_8_8[] = {
		// FrontLeft	FrontRight	Center		LowFreq		SideLeft	SideRight	BackLeft	BackRight
		1.0f,			0.0f,		0.0f,		0.0f,		0.0f,		0.0f,		0.0f,		0.0f,		// FrontLeft
		0.0f,			1.0f,		0.0f,		0.0f,		0.0f,		0.0f,		0.0f,		0.0f,		// FrontRight
		0.0f,			0.0f,		1.0f,		0.0f,		0.0f,		0.0f,		0.0f,		0.0f,		// Center
		0.0f,			0.0f,		0.0f,		1.0f,		0.0f,		0.0f,		0.0f,		0.0f,		// LowFrequency
		0.0f,			0.0f,		0.0f,		0.0f,		1.0f,		0.0f,		0.0f,		0.0f,		// SideLeft
		0.0f,			0.0f,		0.0f,		0.0f,		0.0f,		1.0f,		0.0f,		0.0f,		// SideRight
		0.0f,			0.0f,		0.0f,		0.0f,		0.0f,		0.0f,		1.0f,		0.0f,		// BackLeft
		0.0f,			0.0f,		0.0f,		0.0f,		0.0f,		0.0f,		0.0f,		1.0f,		// BackRight
	};

	static const float* ToSurround[8] = {
		Matrix_1_8,
		Matrix_2_8,
		Matrix_3_8,
		Matrix_4_8,
		Matrix_5_8,
		Matrix_6_8,
		Matrix_7_8,
		Matrix_8_8,
	};


	// Down-mix matrices ------------------------------------------------------

	/** Collection fo down-mix matrices. */
	static const float** DownmixMatrices[] = {
		ToMono,
		ToStereo,
		nullptr,
		nullptr,
		nullptr,
		nullptr,
		nullptr,
		ToSurround,
	};


	/**
	 * Down-mix a sample buffer.
	 *
	 * @param Src The buffer to down-mix from.
	 * @param SrcChannels The number of audio channels in Src.
	 * @param SrcFrames The number of frames in Src.
	 * @param Dest The buffer to down-mix to.
	 * @param DestChannels The number of channels in Dest.
	 * @return true on success, false otherwise.
	 */
	bool Downmix(const float* Src, const uint32 SrcChannels, uint32 SrcFrames, float* Dest, const uint32 DestChannels)
	{
		// select down-mix matrix
		if ((SrcChannels == 0) || (SrcChannels > 8) || (DestChannels == 0) || (DestChannels > 8))
		{
			return false;
		}

		const float* DownmixMatrix = DownmixMatrices[DestChannels - 1][SrcChannels - 1];

		if (DownmixMatrix == nullptr)
		{
			return false;
		}

		// down-mix samples
		while (SrcFrames-- > 0)
		{
			const float* Coeff = DownmixMatrix;

			for (uint32 DestIndex = 0; DestIndex < DestChannels; ++Dest, ++DestIndex)
			{
				*Dest = 0.0f;

				for (uint32 SrcIndex = 0; SrcIndex < SrcChannels; ++Coeff, ++SrcIndex)
				{
					*Dest += Src[SrcIndex] * *Coeff;
				}
			}

			Src += SrcChannels;
		}

		return true;
	}
}


/* FMediaAudioResampler structors
 *****************************************************************************/

FMediaAudioResampler::FMediaAudioResampler()
	: FrameAlpha(0.0f)
	, FrameIndex(MIN_int32)
	, LastFrameIndex(MIN_int32)
	, LastRate(0.0f)
	, InputDuration(FTimespan::Zero())
	, InputFrames(0)
	, InputSampleRate(0)
	, InputTime(FTimespan::Zero())
	, OutputChannels(0)
	, OutputSampleRate(0)
{ }


/* FMediaAudioResampler interface
 *****************************************************************************/

void FMediaAudioResampler::Flush()
{
	ClearInput();

	FrameAlpha = 0.0f;
	FrameIndex = MIN_int32;
	LastFrameIndex = MIN_int32;
	LastRate = 0.0f;
}


uint32 FMediaAudioResampler::Generate(float* Output, FMediaTimeStamp& OutTime, const uint32 FramesRequested, float Rate, FTimespan Time, FMediaAudioSampleSource& SampleSource, uint32& JumpFrame)
{
	JumpFrame = MAX_uint32;
	if ((FramesRequested == 0) || (Output == nullptr) || (OutputSampleRate == 0))
	{
		return 0;
	}

	uint32 FramesGenerated = 0;

	while (FramesGenerated < FramesRequested)
	{
		// request new input buffer
		if (LastFrameIndex == MIN_int32)
		{
			double RateDirection = Rate > 0 ? 1 : -1;
			TRange<FTimespan> ContiguousRange = TRange<FTimespan>::Inclusive(
				InputTime.Time + InputDuration * 0.5 * RateDirection,
				InputTime.Time + InputDuration * 1.5 * RateDirection);

			FMediaTimeStamp PreviousInputTime = InputTime;
			FTimespan PreviousInputDuration = InputDuration;

			// fetch next sample
			TSharedPtr<IMediaAudioSample, ESPMode::ThreadSafe> NextSample;
			SampleSource.Dequeue(NextSample);

			if (!SetInput(NextSample))
			{
				// Keep InputTime/InputDuration to detect jumps, loops, seeks
				InputTime = PreviousInputTime;
				InputDuration = PreviousInputDuration;
				break;
			}

			// Detect jumps in time (looping, fast forwarding, seeking)
			bool bIsContiguous = ContiguousRange.Contains(InputTime.Time);
			if (!bIsContiguous)
			{
				JumpFrame = FramesGenerated;
			}
			UE_LOGF(LogMediaUtils, VeryVerbose, "[AUDIO DEQUEUE] PlayerTime: %ls, SampleTime: %ls, Contiguous: %d, Frames: %d", *Time.ToString(), *InputTime.Time.ToString(), bIsContiguous, NextSample->GetFrames());
		}

		// skip input if too small
		if (FrameIndex >= 0 && FrameIndex >= InputFrames)
		{
			LastFrameIndex = MIN_int32;
			FrameIndex -= InputFrames;
			continue;
		}

		// calculate output time
		const float InputOffset = (FrameIndex + FrameAlpha) / InputFrames;
		OutTime.SetTimeAndIndexValue(InputTime.Time + InputOffset * InputDuration, InputTime.GetIndexValue());

		// invert buffer if rate reversed
		if ((Rate * LastRate) < 0.0f)
		{
			Algo::Reverse(Input);

			FrameIndex = InputFrames - FrameIndex - 1;
			FrameAlpha = 1.0f - FrameAlpha;
			LastRate = Rate;
		}

		// get current & next input frame
		if (FrameIndex != LastFrameIndex)
		{
			const float* InputFrameBuffer = Input.GetData();

			if (FrameIndex == INDEX_NONE)
			{
				if (!ensure((uint32)(Input.Num()) >= OutputChannels))
				{
					ClearInput();
					break;
				}

				// we're still in the last frame of the previous input buffer
				for (uint32 Channel = 0; Channel < OutputChannels; ++Channel)
				{
					NextFrame[Channel] = InputFrameBuffer[Channel];
				}
			}
			else if (FrameIndex == InputFrames - 1)
			{
				if (!ensure(Input.Num() >= ((FrameIndex * OutputChannels) + OutputChannels)))
				{
					ClearInput();
					break;
				}

				// reached the end of the input buffer; cache last frame
				InputFrameBuffer += FrameIndex * OutputChannels;

				for (uint32 Channel = 0; Channel < OutputChannels; ++Channel)
				{
					CurrentFrame[Channel] = InputFrameBuffer[Channel];
				}

				LastFrameIndex = MIN_int32;
				FrameIndex = INDEX_NONE;

				continue;
			}
			else
			{
				if (!ensure(Input.Num() >= ((FrameIndex * OutputChannels) + OutputChannels + OutputChannels)))
				{
					ClearInput();
					break;
				}

				// we're in the current input buffer
				InputFrameBuffer += FrameIndex * OutputChannels;

				for (uint32 Channel = 0; Channel < OutputChannels; ++Channel)
				{
					CurrentFrame[Channel] = InputFrameBuffer[Channel];
				}

				InputFrameBuffer += OutputChannels;

				for (uint32 Channel = 0; Channel < OutputChannels; ++Channel)
				{
					NextFrame[Channel] = InputFrameBuffer[Channel];
				}
			}

			LastFrameIndex = FrameIndex;
		}

		// generate output frame
		for (uint32 OutputIndex = 0; OutputIndex < OutputChannels; ++OutputIndex, ++Output)
		{
			*Output = FMath::Lerp(CurrentFrame[OutputIndex], NextFrame[OutputIndex], FrameAlpha);
		}

		++FramesGenerated;
		
		// update frame alpha
		FrameAlpha += (FMath::Abs(Rate) * InputSampleRate) / (float)OutputSampleRate;
		const int32 FrameAlphaRounded = (int32)FrameAlpha;
		FrameAlpha -= (float)FrameAlphaRounded;
		FrameIndex += FrameAlphaRounded;

		check(FrameAlpha >= 0.0);
		check(FrameAlpha < 1.0f);
	}

	return FramesGenerated;
}


void FMediaAudioResampler::Initialize(const uint32 InOutputChannels, const uint32 InOutputSampleRate)
{
	check((InOutputChannels == 1) || (InOutputChannels == 2) || (InOutputChannels == 8));

	FrameAlpha = 0.0f;
	OutputChannels = InOutputChannels;
	OutputSampleRate = InOutputSampleRate;
}


bool FMediaAudioResampler::SetInput(const TSharedPtr<IMediaAudioSample, ESPMode::ThreadSafe>& Sample)
{
	ClearInput();

	// validate parameters
	if (!Sample.IsValid())
	{
		return false;
	}

	const uint8* Buffer = (const uint8*)Sample->GetBuffer();
	const uint32 NumChannels = Sample->GetChannels();
	const uint32 NumFrames = Sample->GetFrames();
	const uint32 SampleRate = Sample->GetSampleRate();

	if ((Buffer == nullptr) || (NumChannels == 0) || (NumFrames == 0) || (SampleRate == 0))
	{
		return false;
	}

	// initialize input
	const uint32 NumSamples = NumFrames * NumChannels;

	if (Sample->GetFormat() == EMediaAudioSampleFormat::Float)
	{
		if (NumChannels == OutputChannels)
		{
			// copy straight over
			Input.Append((float*)Buffer, NumSamples);
		}
		else
		{
			// down-mix channels
			Input.AddUninitialized(NumFrames * OutputChannels);

			if (!MediaAudioResampler::Downmix((float*)Buffer, NumChannels, NumFrames, Input.GetData(), OutputChannels))
			{
				Input.Reset();
			}
		}
	}
	else
	{
		// convert samples to float
		TArray<float> FloatSamples;

		FloatSamples.AddUninitialized(NumSamples);

		float* FloatData = FloatSamples.GetData();
		const float* FloatDataEnd = FloatData + NumSamples;

		switch (Sample->GetFormat())
		{
		case EMediaAudioSampleFormat::Double:
			while (FloatData < FloatDataEnd)
			{
				*FloatData++ = (float)*(const double*)Buffer;
				Buffer += sizeof(double);
			}
			break;

		case EMediaAudioSampleFormat::Int16:
		{
			Audio::ArrayPcm16ToFloat(MakeArrayView((const int16*)Buffer, NumSamples), MakeArrayView(FloatData, NumSamples));
			break;
		}

		case EMediaAudioSampleFormat::Int32:
			while (FloatData < FloatDataEnd)
			{
				*FloatData++ = (float)*(const int32*)Buffer / 2147483648.0f;
				Buffer += sizeof(int32);
			}
			break;

		case EMediaAudioSampleFormat::Int8:
			while (FloatData < FloatDataEnd)
			{
				*FloatData++ = (float)*(const int8*)Buffer / 128.0f;
				Buffer += sizeof(int8);
			}
			break;

		default:
			return false; // unsupported sample format
		}

		if (NumChannels == OutputChannels)
		{
			// copy straight over
			Input.Append(FloatSamples);
		}
		else
		{
			// down-mix channels
			Input.AddDefaulted(NumFrames * OutputChannels);

			if (!MediaAudioResampler::Downmix(FloatSamples.GetData(), NumChannels, NumFrames, Input.GetData(), OutputChannels))
			{
				Input.Reset();
			}
		}
	}

	// recalculate frame index
	if (FrameIndex == MIN_int32)
	{
		FrameIndex = 0; // first sample
	}
	else if (InputSampleRate > 0)
	{
		FrameAlpha *= (float)InputSampleRate / (float)SampleRate;
		FrameIndex = (int32)FrameAlpha;
		FrameAlpha -= (float)FrameIndex;
	}

	InputDuration = Sample->GetDuration();
	InputFrames = NumFrames;
	InputSampleRate = SampleRate;
	InputTime = Sample->GetTime();

	return true;
}


/* FMediaAudioResampler implementation
 *****************************************************************************/

void FMediaAudioResampler::ClearInput()
{
	Input.Reset();

	InputDuration = FTimespan::Zero();
	InputFrames = 0;
	InputSampleRate = 0;
	InputTime = FMediaTimeStamp(FTimespan::Zero());

	LastRate = 0.0f;
}
