// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaCaptureSupport.h"

#include "IMediaModule.h"
#include "Modules/ModuleManager.h"


namespace MediaCaptureSupport
{
	void EnumerateAudioCaptureDevices(TArray<FMediaCaptureDeviceInfo>& OutDeviceInfos)
	{
		auto MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media");

		if (MediaModule == nullptr)
		{
			return;
		}

		const TArray<IMediaCaptureSupport*>& CaptureSupports = MediaModule->GetCaptureSupports();

		for (const auto& CaptureSupport : CaptureSupports)
		{
			CaptureSupport->EnumerateAudioCaptureDevices(OutDeviceInfos);
		}
	}


	void EnumerateVideoCaptureDevices(TArray<FMediaCaptureDeviceInfo>& OutDeviceInfos)
	{
		auto MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media");

		if (MediaModule == nullptr)
		{
			return;
		}

		const TArray<IMediaCaptureSupport*>& CaptureSupports = MediaModule->GetCaptureSupports();

		for (const auto& CaptureSupport : CaptureSupports)
		{
			CaptureSupport->EnumerateVideoCaptureDevices(OutDeviceInfos);
		}
	}
}
