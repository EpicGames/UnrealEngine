// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaDelegates.h"
#include "MediaUtilsPrivate.h"

#if UE_MEDIAUTILS_DEVELOPMENT_DELEGATE

FMediaDelegates::FOnSampleDiscardedDelegate FMediaDelegates::OnSampleDiscarded_RenderThread;
FMediaDelegates::FOnPreSampleRenderDelegate FMediaDelegates::OnPreSampleRender_RenderThread;

#endif //UE_MEDIAUTILS_DEVELOPMENT_DELEGATE

FMediaDelegates::FOnSampleRenderedDelegate FMediaDelegates::OnSampleRendered_RenderThread;
FMediaDelegates::FOnMediaTextureResourceReleasedDelegate FMediaDelegates::OnMediaTextureResourceReleased_RenderThread;