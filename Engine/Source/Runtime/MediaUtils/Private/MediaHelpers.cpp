// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaHelpers.h"

#include "IMediaControls.h"
#include "IMediaEventSink.h"
#include "IMediaTracks.h"


namespace MediaUtils
{
	FString EventToString(EMediaEvent Event)
	{
		switch (Event)
		{
		case EMediaEvent::MediaBuffering: return TEXT("MediaBuffering");
		case EMediaEvent::MediaBufferingComplete: return TEXT("MediaBufferingComplete");
		case EMediaEvent::MediaClosed: return TEXT("MediaClosed");
		case EMediaEvent::MediaConnecting: return TEXT("MediaConnecting");
		case EMediaEvent::MediaOpened: return TEXT("MediaOpened");
		case EMediaEvent::MediaOpenFailed: return TEXT("MediaOpenFailed");
		case EMediaEvent::PlaybackEndReached: return TEXT("PlaybackEndReached");
		case EMediaEvent::PlaybackResumed: return TEXT("PlaybackResumed");
		case EMediaEvent::PlaybackSuspended: return TEXT("PlaybackSuspended");
		case EMediaEvent::SeekCompleted: return TEXT("SeekCompleted");
		case EMediaEvent::TracksChanged: return TEXT("TracksChanged");
		case EMediaEvent::MetadataChanged: return TEXT("MetadataChanged");

		case EMediaEvent::Internal_PurgeVideoSamplesHint: return TEXT("Internal: PurgeVideoSamplesHint");
		case EMediaEvent::Internal_VideoSamplesAvailable: return TEXT("Internal: VideoSamplesAvailable");
		case EMediaEvent::Internal_VideoSamplesUnavailable: return TEXT("Internal: VideoSamplesUnavailable");
		case EMediaEvent::Internal_AudioSamplesAvailable: return TEXT("Internal: AudioSamplesAvailable");
		case EMediaEvent::Internal_AudioSamplesUnavailable: return TEXT("Internal: AudioSamplesUnavailable");
		case EMediaEvent::Internal_RequestVideoSampleRefresh: return TEXT("Internal: RequestVideoSampleRefresh");


		// no default case; all cases must be implemented!
		}

		return TEXT("Unknown");
	}


	FString StateToString(EMediaState State)
	{
		switch (State)
		{
		case EMediaState::Closed: return TEXT("Closed");
		case EMediaState::Error: return TEXT("Error");
		case EMediaState::Paused: return TEXT("Paused");
		case EMediaState::Playing: return TEXT("Playing");
		case EMediaState::Preparing: return TEXT("Preparing");
		case EMediaState::Stopped: return TEXT("Stopped");

		// no default case; all cases must be implemented!
		}

		return TEXT("Unknown");
	}


	FString TrackTypeToString(EMediaTrackType TrackType)
	{
		switch (TrackType)
		{
		case EMediaTrackType::Audio: return TEXT("Audio");
		case EMediaTrackType::Caption: return TEXT("Caption");
		case EMediaTrackType::Metadata: return TEXT("Metadata");
		case EMediaTrackType::Script: return TEXT("Script");
		case EMediaTrackType::Subtitle: return TEXT("Subtitle");
		case EMediaTrackType::Text: return TEXT("Text");
		case EMediaTrackType::Video: return TEXT("Video");

		// no default case; all cases must be implemented!
		}

		return TEXT("Unknown");
	}
}
