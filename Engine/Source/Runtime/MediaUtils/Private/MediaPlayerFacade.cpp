// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaPlayerFacade.h"
#include "MediaUtilsPrivate.h"

#include "HAL/PlatformMath.h"
#include "HAL/PlatformProcess.h"
#include "IMediaCache.h"
#include "IMediaControls.h"
#include "IMediaModule.h"
#include "IMediaOptions.h"
#include "IMediaPlayer.h"
#include "IMediaPlayerFactory.h"
#include "IMediaSamples.h"
#include "IMediaAudioSample.h"
#include "IMediaTextureSample.h"
#include "IMediaOverlaySample.h"
#include "IMediaTracks.h"
#include "IMediaView.h"
#include "IMediaTicker.h"
#include "MediaPlayerOptions.h"
#include "Math/NumericLimits.h"
#include "Misc/CoreMisc.h"
#include "Misc/ScopeLock.h"
#include "Modules/ModuleManager.h"

#include "MediaHelpers.h"
#include "MediaSampleCache.h"
#include "MediaSampleQueueDepths.h"
#include "MediaSampleQueue.h"

#include "Async/Async.h"

#include <algorithm>

#define MEDIAPLAYERFACADE_DISABLE_BLOCKING 0
#define MEDIAPLAYERFACADE_TRACE_SINKOVERFLOWS 0


/** Time spent in media player facade closing media. */
DECLARE_CYCLE_STAT(TEXT("MediaUtils MediaPlayerFacade Close"), STAT_MediaUtils_FacadeClose, STATGROUP_Media);

/** Time spent in media player facade opening media. */
DECLARE_CYCLE_STAT(TEXT("MediaUtils MediaPlayerFacade Open"), STAT_MediaUtils_FacadeOpen, STATGROUP_Media);

/** Time spent in media player facade event processing. */
DECLARE_CYCLE_STAT(TEXT("MediaUtils MediaPlayerFacade ProcessEvent"), STAT_MediaUtils_FacadeProcessEvent, STATGROUP_Media);

/** Time spent in media player facade fetch tick. */
DECLARE_CYCLE_STAT(TEXT("MediaUtils MediaPlayerFacade TickFetch"), STAT_MediaUtils_FacadeTickFetch, STATGROUP_Media);

/** Time spent in media player facade input tick. */
DECLARE_CYCLE_STAT(TEXT("MediaUtils MediaPlayerFacade TickInput"), STAT_MediaUtils_FacadeTickInput, STATGROUP_Media);

/** Time spent in media player facade output tick. */
DECLARE_CYCLE_STAT(TEXT("MediaUtils MediaPlayerFacade TickOutput"), STAT_MediaUtils_FacadeTickOutput, STATGROUP_Media);

/** Time spent in media player facade high frequency tick. */
DECLARE_CYCLE_STAT(TEXT("MediaUtils MediaPlayerFacade TickTickable"), STAT_MediaUtils_FacadeTickTickable, STATGROUP_Media);

/** Player time on main thread during last fetch tick. */
DECLARE_FLOAT_COUNTER_STAT(TEXT("MediaPlayerFacade PlaybackTime"), STAT_MediaUtils_FacadeTime, STATGROUP_Media);

/** Number of video samples currently in the queue. */
DECLARE_DWORD_COUNTER_STAT(TEXT("MediaPlayerFacade NumVideoSamples"), STAT_MediaUtils_FacadeNumVideoSamples, STATGROUP_Media);

/** Number of audio samples currently in the queue. */
DECLARE_DWORD_COUNTER_STAT(TEXT("MediaPlayerFacade NumAudioSamples"), STAT_MediaUtils_FacadeNumAudioSamples, STATGROUP_Media);

/** Number of purged video samples */
DECLARE_DWORD_COUNTER_STAT(TEXT("MediaPlayerFacade NumPurgedVideoSamples"), STAT_MediaUtils_FacadeNumPurgedVideoSamples, STATGROUP_Media);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("MediaPlayerFacade TotalPurgedVideoSamples"), STAT_MediaUtils_FacadeTotalPurgedVideoSamples, STATGROUP_Media);

/** Number of purged subtitle samples */
DECLARE_DWORD_COUNTER_STAT(TEXT("MediaPlayerFacade NumPurgedSubtitleSamples"), STAT_MediaUtils_FacadeNumPurgedSubtitleSamples, STATGROUP_Media);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("MediaPlayerFacade TotalPurgedSubtitleSamples"), STAT_MediaUtils_FacadeTotalPurgedSubtitleSamples, STATGROUP_Media);

/** Number of purged caption samples */
DECLARE_DWORD_COUNTER_STAT(TEXT("MediaPlayerFacade NumPurgedCaptionSamples"), STAT_MediaUtils_FacadeNumPurgedCaptionSamples, STATGROUP_Media);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("MediaPlayerFacade TotalPurgedCaptionSamples"), STAT_MediaUtils_FacadeTotalPurgedCaptionSamples, STATGROUP_Media);

/* Some constants
*****************************************************************************/

static const double kMaxTimeSinceFrameStart = 0.300;			// max seconds we allow between the start of the frame and the player facade timing computations (to catch suspended apps & debugging)
static const double kMaxTimeSinceAudioTimeSampling = 0.250;		// max seconds we allow to have passed between the last audio timing sampling and the player facade timing computations (to catch suspended apps & debugging - some platforms do update audio at a farily low rate: hence the big tollerance)
static const double kOutdatedVideoSamplesTolerance = 0.080;		// seconds video samples are allowed to be "too old" to stay in the player's output queue despite of calculations indicating they need to go
static const double kOutdatedSubtitleSamplesTolerance = 1.0;	// seconds subtitle samples are allowed to be "too old" to stay in the player's output queue despite of calculations indicating they need to go
static const double kOutdatedSamplePurgeRange = 1.0;			// milliseconds for pseudo DT timespan used with async purging of outdated video samples
static const int32	kMinFramesInVideoQueueToPurge = 3;			// we only consider purging any old frames from the video queue if more than these are present (to not kill a slow playback entirely)
static const int32	kMinFramesInSubtitleQueueToPurge = 3;		// we only consider purging any old frames from the subtitle queue if more than these are present (to not kill a slow playback entirely)
static const int32	kMinFramesInCaptionQueueToPurge = 3;		// we only consider purging any old frames from the caption queue if more than these are present (to not kill a slow playback entirely)

/* Cvars
*****************************************************************************/

namespace UE::MediaUtils::Private
{

#if !UE_BUILD_SHIPPING

TAutoConsoleVariable<bool> CVarTestForcePlayerCreateFailed(
	TEXT("m.Test.ForcePlayerCreateFailed"),
	false,
	TEXT("Whether force media player creation to fail."),
	ECVF_ReadOnly
);

#endif // !UE_BUILD_SHIPPING

float GBlockOnFetchTimeout = 10.0f;
static FAutoConsoleVariableRef CVarMediaUtilsBlockOnFetchTimeout(
	TEXT("MediaUtils.BlockOnFetchTimeout"),
	GBlockOnFetchTimeout,
	TEXT("Maximum time that TickInput/Fetch will block waiting for samples (in seconds).\n")
);

}

/* Local helpers
*****************************************************************************/

namespace MediaPlayerFacade
{
	const FTimespan AudioPreroll = FTimespan::FromSeconds(1.0);
	const FTimespan MetadataPreroll = FTimespan::FromSeconds(1.0);
}

static FTimespan WrappedModulo(FTimespan Time, FTimespan Duration)
{
	return Duration > FTimespan::Zero() ? (Time >= FTimespan::Zero() ? (Time % Duration) : (Duration + (Time % Duration))) : Time;
}

static bool IsDurationValidAndFinite(FTimespan Duration)
{
	return (Duration != FTimespan::Zero() && Duration.GetTicks() != TNumericLimits<int64>::Max());
}

/* FMediaPlayerFacade structors
*****************************************************************************/

FMediaPlayerFacade::FMediaPlayerFacade(TWeakObjectPtr<UMediaPlayer> InMediaPlayer)
	: TimeDelay(FTimespan::Zero())
	, BlockOnRange(this)
	, Cache(new FMediaSampleCache)
	, LastRate(0.0f)
	, CurrentRate(0.0f)
	, bHaveActiveAudio(false)
	, VideoSampleAvailability(-1)
	, AudioSampleAvailability(-1)
	, bAreEventsSafeForAnyThread(false)
	, MediaPlayer(InMediaPlayer)
{
	BlockOnRangeDisabled = false;

	MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media");
	bDidRecentPlayerHaveError = false;

	ResetTracks();
}


FMediaPlayerFacade::~FMediaPlayerFacade()
{
	FMediaSampleSinkEventData Data;
	Data.Detached.MediaPlayer = MediaPlayer.Get();
	SendSinkEvent(EMediaSampleSinkEvent::Detached, Data);

	if (Player.IsValid())
	{
		{
			FScopeLock Lock(&CriticalSection);
			Player->Close();
		}
		NotifyLifetimeManagerDelegate_PlayerClosed();

		DestroyPlayer();
	}

	delete Cache;
	Cache = nullptr;
}


/* FMediaPlayerFacade interface
*****************************************************************************/

void FMediaPlayerFacade::AddAudioSampleSink(const TSharedRef<FMediaAudioSampleSink, ESPMode::ThreadSafe>& SampleSink)
{
	FScopeLock Lock(&CriticalSection);
	AudioSampleSinks.Add(SampleSink);
	PrimaryAudioSink = AudioSampleSinks.GetPrimaryAudioSink();
}


void FMediaPlayerFacade::AddCaptionSampleSink(const TSharedRef<FMediaOverlaySampleSink, ESPMode::ThreadSafe>& SampleSink)
{
	CaptionSampleSinks.Add(SampleSink);
}


void FMediaPlayerFacade::AddMetadataSampleSink(const TSharedRef<FMediaBinarySampleSink, ESPMode::ThreadSafe>& SampleSink)
{
	FScopeLock Lock(&CriticalSection);
	MetadataSampleSinks.Add(SampleSink);
}


void FMediaPlayerFacade::AddSubtitleSampleSink(const TSharedRef<FMediaOverlaySampleSink, ESPMode::ThreadSafe>& SampleSink)
{
	SubtitleSampleSinks.Add(SampleSink);
}


void FMediaPlayerFacade::AddVideoSampleSink(const TSharedRef<FMediaTextureSampleSink, ESPMode::ThreadSafe>& SampleSink)
{
	VideoSampleSinks.Add(SampleSink);
}


bool FMediaPlayerFacade::CanPause() const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		return false;
	}
	return CurrentPlayer->GetControls().CanControl(EMediaControl::Pause);
}


bool FMediaPlayerFacade::CanPlayUrl(const FString& Url, const IMediaOptions* Options)
{
	if (MediaModule == nullptr)
	{
		return false;
	}

	const FString RunningPlatformName(FPlatformProperties::IniPlatformName());
	const TArray<IMediaPlayerFactory*>& PlayerFactories = MediaModule->GetPlayerFactories();

	for (IMediaPlayerFactory* Factory : PlayerFactories)
	{
		if (Factory->SupportsPlatform(RunningPlatformName) && Factory->CanPlayUrl(Url, Options))
		{
			return true;
		}
	}

	return false;
}


bool FMediaPlayerFacade::CanResume() const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		return false;
	}
	return CurrentPlayer->GetControls().CanControl(EMediaControl::Resume);
}


bool FMediaPlayerFacade::CanScrub() const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		return false;
	}
	return CurrentPlayer->GetControls().CanControl(EMediaControl::Scrub);
}


bool FMediaPlayerFacade::CanSeek() const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		return false;
	}
	return CurrentPlayer->GetControls().CanControl(EMediaControl::Seek);
}


bool FMediaPlayerFacade::SupportsPlaybackTimeRange() const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	return CurrentPlayer.IsValid() ? CurrentPlayer->GetControls().CanControl(EMediaControl::PlaybackRange) : false;
}


void FMediaPlayerFacade::Close()
{
	SCOPE_CYCLE_COUNTER(STAT_MediaUtils_FacadeClose);

	if (CurrentUrl.IsEmpty())
	{
		return;
	}

	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (CurrentPlayer.IsValid())
	{
		{
			FScopeLock Lock(&CriticalSection);
			CurrentPlayer->Close();
		}
		NotifyLifetimeManagerDelegate_PlayerClosed();
	}

	Flush();
	ReInit();
	BlockOnRange.Reset();
	bDidRecentPlayerHaveError = false;
}


uint32 FMediaPlayerFacade::GetAudioTrackChannels(int32 TrackIndex, int32 FormatIndex) const
{
	FMediaAudioTrackFormat Format;
	return GetAudioTrackFormat(TrackIndex, FormatIndex, Format) ? Format.NumChannels : 0;
}


uint32 FMediaPlayerFacade::GetAudioTrackSampleRate(int32 TrackIndex, int32 FormatIndex) const
{
	FMediaAudioTrackFormat Format;
	return GetAudioTrackFormat(TrackIndex, FormatIndex, Format) ? Format.SampleRate : 0;
}


FString FMediaPlayerFacade::GetAudioTrackType(int32 TrackIndex, int32 FormatIndex) const
{
	FMediaAudioTrackFormat Format;
	return GetAudioTrackFormat(TrackIndex, FormatIndex, Format) ? Format.TypeName : FString();
}


FTimespan FMediaPlayerFacade::GetDuration() const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		return FTimespan::Zero();
	}
	return CurrentPlayer->GetControls().GetDuration();
}


const FGuid& FMediaPlayerFacade::GetGuid()
{
	return PlayerGuid;
}


FString FMediaPlayerFacade::GetInfo() const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		return FString();
	}
	return CurrentPlayer->GetInfo();
}


FVariant FMediaPlayerFacade::GetMediaInfo(FName InfoName) const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		return FVariant();
	}
	return CurrentPlayer->GetMediaInfo(InfoName);
}


FText FMediaPlayerFacade::GetMediaName() const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		return FText::GetEmpty();
	}
	return CurrentPlayer->GetMediaName();
}


TSharedPtr<TMap<FString, TArray<TUniquePtr<IMediaMetadataItem>>>, ESPMode::ThreadSafe> FMediaPlayerFacade::GetMediaMetadata() const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		return nullptr;
	}
	return CurrentPlayer->GetMediaMetadata();
}


int32 FMediaPlayerFacade::GetNumTracks(EMediaTrackType TrackType) const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		return 0;
	}
	return CurrentPlayer->GetTracks().GetNumTracks(TrackType);
}


int32 FMediaPlayerFacade::GetNumTrackFormats(EMediaTrackType TrackType, int32 TrackIndex) const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		return 0;
	}
	return CurrentPlayer->GetTracks().GetNumTrackFormats(TrackType, TrackIndex);
}


FName FMediaPlayerFacade::GetPlayerName() const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		return NAME_None;
	}
	return MediaModule->GetPlayerFactory(CurrentPlayer->GetPlayerPluginGUID())->GetPlayerName();
}


float FMediaPlayerFacade::GetRate() const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		return 0.0f;
	}
	return CurrentPlayer->GetControls().GetRate();
}


FString FMediaPlayerFacade::GetStats() const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		return FString();
	}
	return CurrentPlayer->GetStats();
}


TRangeSet<float> FMediaPlayerFacade::GetSupportedRates(bool Unthinned) const
{
	const EMediaRateThinning Thinning = Unthinned ? EMediaRateThinning::Unthinned : EMediaRateThinning::Thinned;

	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		return TRangeSet<float>();
	}
	return CurrentPlayer->GetControls().GetSupportedRates(Thinning);
}


bool FMediaPlayerFacade::HaveVideoPlayback() const
{
	return VideoSampleSinks.Num() && (GetSelectedTrack(EMediaTrackType::Video) != INDEX_NONE);
}


bool FMediaPlayerFacade::HaveAudioPlayback() const
{
	return PrimaryAudioSink.IsValid() && (GetSelectedTrack(EMediaTrackType::Audio) != INDEX_NONE);
}


FTimespan FMediaPlayerFacade::GetTime() const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);

	if (!CurrentPlayer.IsValid())
	{
		return FTimespan::Zero(); // no media opened
	}

	FTimespan Result;

	if (CurrentPlayer->GetPlayerFeatureFlag(IMediaPlayer::EFeatureFlag::UsePlaybackTimingV2))
	{
		// New style: framework controls timing - we use GetTimeStamp() and return the legacy part of the value
		FMediaTimeStamp TimeStamp = GetTimeStamp();
		return TimeStamp.IsValid() ? TimeStamp.Time : FTimespan::Zero();
	}
	else
	{
		// Old style: ask the player for timing
		Result = CurrentPlayer->GetControls().GetTime() - TimeDelay;
		if (Result.GetTicks() < 0)
		{
			Result = FTimespan::Zero();
		}
	}

	return Result;
}


FMediaTimeStamp FMediaPlayerFacade::GetTimeStamp() const
{
	return GetTimeStampInternal(false);
}


FMediaTimeStamp FMediaPlayerFacade::GetDisplayTimeStamp() const
{
	return GetTimeStampInternal(true);
}

TOptional<FTimecode> FMediaPlayerFacade::GetVideoTimecode() const
{
	FScopeLock Lock(&TimeValueLock);
	return MostRecentlyDeliveredVideoFrameTimecode;
}

TRange<FMediaTimeStamp> FMediaPlayerFacade::GetLastProcessedVideoSampleTimeRange() const
{
	FScopeLock Lock(&TimeValueLock);
	return LastVideoSampleProcessedTimeRange;
}

FMediaTimeStamp FMediaPlayerFacade::GetTimeStampInternal(bool bForDisplay) const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		return FMediaTimeStamp();
	}

	if (!CurrentPlayer->GetPlayerFeatureFlag(IMediaPlayer::EFeatureFlag::UsePlaybackTimingV2))
	{
		// Make sure we can return values for V1 players...
		return FMediaTimeStamp(GetTime());
	}

	FScopeLock Lock(&TimeValueLock);
	// Check if the value is for display purposes. If so: do we seek right now?
	if (bForDisplay && IsSeeking())
	{
		return SeekTargetTime;
	}

	// Check if there are video samples present or presence is unknown.
	// Only when we know for sure that there are none because the existing video stream has ended do we set this to false.
	bool bHaveVideoSamples = VideoSampleAvailability != 0;

	if (HaveVideoPlayback() && bHaveVideoSamples)
	{
		/*
			Returning the precise time of the sample returned during TickFetch()
		*/
		return bForDisplay ? CurrentFrameVideoDisplayTimeStamp : CurrentFrameVideoTimeStamp;
	}
	else if (HaveAudioPlayback())
	{
		/*
			We grab the last processed audio sample timestamp when it gets passed out to the sink(s) and keep it
			as "the value" for the frame (on the gamethread) -- an approximation, but better then having it return
			new values each time its called in one and the same frame...
		*/
		return CurrentFrameAudioTimeStamp;
	}

	// we assume video and/or audio to be present in any stream we play - otherwise: no time info
	// (at least for now)
	return FMediaTimeStamp();
}


FText FMediaPlayerFacade::GetTrackDisplayName(EMediaTrackType TrackType, int32 TrackIndex) const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		return FText::GetEmpty();
	}
	return CurrentPlayer->GetTracks().GetTrackDisplayName((EMediaTrackType)TrackType, TrackIndex);
}


int32 FMediaPlayerFacade::GetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex) const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		return INDEX_NONE;
	}
	return CurrentPlayer->GetTracks().GetTrackFormat((EMediaTrackType)TrackType, TrackIndex);
}


FString FMediaPlayerFacade::GetTrackLanguage(EMediaTrackType TrackType, int32 TrackIndex) const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		return FString();
	}
	return CurrentPlayer->GetTracks().GetTrackLanguage((EMediaTrackType)TrackType, TrackIndex);
}

TOptional<FTimespan> FMediaPlayerFacade::GetTrackDuration(EMediaTrackType TrackType, int32 TrackIndex) const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		return TOptional<FTimespan>();
	}
	return CurrentPlayer->GetTracks().GetTrackDuration((EMediaTrackType)TrackType, TrackIndex);
}


float FMediaPlayerFacade::GetVideoTrackAspectRatio(int32 TrackIndex, int32 FormatIndex) const
{
	FMediaVideoTrackFormat Format;
	return (GetVideoTrackFormat(TrackIndex, FormatIndex, Format) && (Format.Dim.Y != 0)) ? ((float)(Format.Dim.X) / (float)Format.Dim.Y) : 0.0f;
}


FIntPoint FMediaPlayerFacade::GetVideoTrackDimensions(int32 TrackIndex, int32 FormatIndex) const
{
	FMediaVideoTrackFormat Format;
	return GetVideoTrackFormat(TrackIndex, FormatIndex, Format) ? Format.Dim : FIntPoint::ZeroValue;
}


float FMediaPlayerFacade::GetVideoTrackFrameRate(int32 TrackIndex, int32 FormatIndex) const
{
	FMediaVideoTrackFormat Format;
	return GetVideoTrackFormat(TrackIndex, FormatIndex, Format) ? Format.FrameRate : 0.0f;
}


TRange<float> FMediaPlayerFacade::GetVideoTrackFrameRates(int32 TrackIndex, int32 FormatIndex) const
{
	FMediaVideoTrackFormat Format;
	return GetVideoTrackFormat(TrackIndex, FormatIndex, Format) ? Format.FrameRates : TRange<float>::Empty();
}


FString FMediaPlayerFacade::GetVideoTrackType(int32 TrackIndex, int32 FormatIndex) const
{
	FMediaVideoTrackFormat Format;
	return GetVideoTrackFormat(TrackIndex, FormatIndex, Format) ? Format.TypeName : FString();
}


bool FMediaPlayerFacade::GetViewField(float& OutHorizontal, float& OutVertical) const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		return false;
	}
	return CurrentPlayer->GetView().GetViewField(OutHorizontal, OutVertical);
}


bool FMediaPlayerFacade::GetViewOrientation(FQuat& OutOrientation) const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		return false;
	}
	return CurrentPlayer->GetView().GetViewOrientation(OutOrientation);
}


bool FMediaPlayerFacade::HasError() const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		return bDidRecentPlayerHaveError;
	}
	return (CurrentPlayer->GetControls().GetState() == EMediaState::Error);
}


bool FMediaPlayerFacade::IsBuffering() const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		return false;
	}
	return EnumHasAnyFlags(CurrentPlayer->GetControls().GetStatus(), EMediaStatus::Buffering);
}


bool FMediaPlayerFacade::IsConnecting() const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		return false;
	}
	return EnumHasAnyFlags(CurrentPlayer->GetControls().GetStatus(), EMediaStatus::Connecting);
}


bool FMediaPlayerFacade::IsLooping() const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		return false;
	}
	return CurrentPlayer->GetControls().IsLooping();
}


bool FMediaPlayerFacade::IsPaused() const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		return false;
	}
	return (CurrentPlayer->GetControls().GetState() == EMediaState::Paused);
}


bool FMediaPlayerFacade::IsPlaying() const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		return false;
	}
	return (CurrentPlayer->GetControls().GetState() == EMediaState::Playing);
}


bool FMediaPlayerFacade::IsPreparing() const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		return false;
	}
	return (CurrentPlayer->GetControls().GetState() == EMediaState::Preparing);
}

bool FMediaPlayerFacade::IsClosed() const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		return false;
	}
	return (CurrentPlayer->GetControls().GetState() == EMediaState::Closed);
}

bool FMediaPlayerFacade::IsReady() const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		return false;
	}

	EMediaState State = CurrentPlayer->GetControls().GetState();
	return ((State != EMediaState::Closed) &&
		(State != EMediaState::Error) &&
		(State != EMediaState::Preparing));
}

// ----------------------------------------------------------------------------------------------------------------------------------------------

class FMediaPlayerLifecycleManagerDelegateOpenRequest : public IMediaPlayerLifecycleManagerDelegate::IOpenRequest
{
public:
	FMediaPlayerLifecycleManagerDelegateOpenRequest(const FString& InUrl, const IMediaOptions* InOptions, const FMediaPlayerOptions* InPlayerOptions, IMediaPlayerFactory* InPlayerFactory, TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> InReusedPlayer, bool bInWillCreatePlayer, uint32 InWillUseNewResources)
		: Url(InUrl)
		, Options(InOptions)
		, OptionsObject(InOptions ? InOptions->ToUObject() : nullptr)
		, PlayerFactory(InPlayerFactory)
		, ReusedPlayer(InReusedPlayer)
		, bWillCreatePlayer(bInWillCreatePlayer)
		, NewResources(InWillUseNewResources)
	{
		if (InPlayerOptions)
		{
			PlayerOptions = *InPlayerOptions;
		}
	}

	virtual const FString& GetUrl() const override
	{
		return Url;
	}

	virtual const IMediaOptions* GetOptions() const override
	{
		return OptionsObject.IsStale() ? nullptr : Options;
	}

	virtual const FMediaPlayerOptions* GetPlayerOptions() const override
	{
		return PlayerOptions.IsSet() ? &PlayerOptions.GetValue() : nullptr;
	}

	virtual IMediaPlayerFactory* GetPlayerFactory() const override
	{
		return PlayerFactory;
	}

	virtual bool WillCreateNewPlayer() const
	{
		return bWillCreatePlayer;
	}

	virtual bool WillUseNewResources(uint32 ResourceFlags) const
	{
		return !!(NewResources & ResourceFlags);
	}

	const TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe>& GetReusedPlayer() const
	{
		return ReusedPlayer;
	}

private:
	FString Url;
	const IMediaOptions* Options;
	FWeakObjectPtr OptionsObject;
	TOptional<FMediaPlayerOptions> PlayerOptions;
	IMediaPlayerFactory* PlayerFactory;
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> ReusedPlayer;
	bool bWillCreatePlayer;
	uint32 NewResources;
};

class FMediaPlayerLifecycleManagerDelegateControl : public IMediaPlayerLifecycleManagerDelegate::IControl, public TSharedFromThis<FMediaPlayerLifecycleManagerDelegateControl, ESPMode::ThreadSafe>
{
public:
	FMediaPlayerLifecycleManagerDelegateControl(TWeakPtr<FMediaPlayerFacade, ESPMode::ThreadSafe> InFacade) : Facade(InFacade), InstanceID(~0), SubmittedRequest(false) {}

	virtual ~FMediaPlayerLifecycleManagerDelegateControl()
	{
		if (!SubmittedRequest)
		{
			if (TSharedPtr<FMediaPlayerFacade, ESPMode::ThreadSafe> PinnedFacade = Facade.Pin())
			{
				PinnedFacade->ReceiveMediaEvent(EMediaEvent::MediaOpenFailed);
			}
		}
	}

	virtual bool SubmitOpenRequest(IMediaPlayerLifecycleManagerDelegate::IOpenRequestRef&& OpenRequest) override
	{
		if (TSharedPtr<FMediaPlayerFacade, ESPMode::ThreadSafe> PinnedFacade = Facade.Pin())
		{
			const FMediaPlayerLifecycleManagerDelegateOpenRequest* OR = static_cast<const FMediaPlayerLifecycleManagerDelegateOpenRequest*>(OpenRequest.Get());
			if (PinnedFacade->ContinueOpen(AsShared(), OR->GetUrl(), OR->GetOptions(), OR->GetPlayerOptions(), OR->GetPlayerFactory(), OR->GetReusedPlayer(), OR->WillCreateNewPlayer(), InstanceID))
			{
				SubmittedRequest = true;
			}
			//note: we return "true" in all cases in which we were able to get to call "ContinueOpen". Failures in here will be messaged to the delegate using the OnMediaPlayerCreateFailed() method
			// (returning true here allows for capturing an unlikely early death of the facade while protecting us from double-handling the failure of the creation in the delegate)
			return true;
		}
		return false;
	}

	virtual TSharedPtr<FMediaPlayerFacade, ESPMode::ThreadSafe> GetFacade() const override
	{
		return Facade.Pin();
	}

	virtual uint64 GetMediaPlayerInstanceID() const override
	{
		return InstanceID;
	}

	void SetInstanceID(uint64 InInstanceID)
	{
		InstanceID = InInstanceID;
	}

	void Reset()
	{
		SubmittedRequest = true;
	}

private:
	TWeakPtr<FMediaPlayerFacade, ESPMode::ThreadSafe> Facade;
	uint64 InstanceID;
	bool SubmittedRequest;
};

// ----------------------------------------------------------------------------------------------------------------------------------------------

bool FMediaPlayerFacade::NotifyLifetimeManagerDelegate_PlayerOpen(IMediaPlayerLifecycleManagerDelegate::IControlRef& NewLifecycleManagerDelegateControl, const FString& Url, const IMediaOptions* Options, const FMediaPlayerOptions* PlayerOptions, IMediaPlayerFactory* PlayerFactory, bool bWillCreatePlayer, uint32 WillUseNewResources, uint64 NewPlayerInstanceID)
{
	check(IsInGameThread() || IsInSlateThread());

	if (IMediaPlayerLifecycleManagerDelegate* Delegate = MediaModule->GetPlayerLifecycleManagerDelegate())
	{
		NewLifecycleManagerDelegateControl = MakeShared<FMediaPlayerLifecycleManagerDelegateControl, ESPMode::ThreadSafe>(AsShared());
		if (NewLifecycleManagerDelegateControl.IsValid())
		{
			// Set instance ID we will use for a new player if we get the go-ahead to create it (old ID if player is about to be reused)
			static_cast<FMediaPlayerLifecycleManagerDelegateControl*>(NewLifecycleManagerDelegateControl.Get())->SetInstanceID(NewPlayerInstanceID);

			IMediaPlayerLifecycleManagerDelegate::IOpenRequestRef OpenRequest(new FMediaPlayerLifecycleManagerDelegateOpenRequest(Url, Options, PlayerOptions, PlayerFactory, !bWillCreatePlayer ? Player : TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe>(), bWillCreatePlayer, WillUseNewResources));
			if (OpenRequest.IsValid())
			{
				if (Delegate->OnMediaPlayerOpen(NewLifecycleManagerDelegateControl, OpenRequest))
				{
					return true;
				}
			}
			static_cast<FMediaPlayerLifecycleManagerDelegateControl*>(NewLifecycleManagerDelegateControl.Get())->Reset();
		}
	}
	return false;
}

bool FMediaPlayerFacade::NotifyLifetimeManagerDelegate_PlayerCreated()
{
	check(IsInGameThread() || IsInSlateThread());
	check(Player.IsValid());

	if (LifecycleManagerDelegateControl.IsValid())
	{
		if (IMediaPlayerLifecycleManagerDelegate* Delegate = MediaModule->GetPlayerLifecycleManagerDelegate())
		{
			Delegate->OnMediaPlayerCreated(LifecycleManagerDelegateControl);
			return true;
		}
	}
	return false;
}

bool FMediaPlayerFacade::NotifyLifetimeManagerDelegate_PlayerCreateFailed()
{
	check(IsInGameThread() || IsInSlateThread());

	if (LifecycleManagerDelegateControl.IsValid())
	{
		if (IMediaPlayerLifecycleManagerDelegate* Delegate = MediaModule->GetPlayerLifecycleManagerDelegate())
		{
			Delegate->OnMediaPlayerCreateFailed(LifecycleManagerDelegateControl);
			return true;
		}
	}
	return false;
}

bool FMediaPlayerFacade::NotifyLifetimeManagerDelegate_PlayerClosed()
{
	check(IsInGameThread() || IsInSlateThread());

	if (LifecycleManagerDelegateControl.IsValid())
	{
		if (IMediaPlayerLifecycleManagerDelegate* Delegate = MediaModule->GetPlayerLifecycleManagerDelegate())
		{
			Delegate->OnMediaPlayerClosed(LifecycleManagerDelegateControl);
			return true;
		}
	}
	return false;
}

bool FMediaPlayerFacade::NotifyLifetimeManagerDelegate_PlayerDestroyed()
{
	check(IsInGameThread() || IsInSlateThread());

	if (LifecycleManagerDelegateControl.IsValid())
	{
		if (IMediaPlayerLifecycleManagerDelegate* Delegate = MediaModule->GetPlayerLifecycleManagerDelegate())
		{
			Delegate->OnMediaPlayerDestroyed(LifecycleManagerDelegateControl);
			return true;
		}
	}
	return false;
}

bool FMediaPlayerFacade::NotifyLifetimeManagerDelegate_PlayerResourcesReleased(uint32 ResourceFlags)
{
	check(IsInGameThread() || IsInSlateThread());

	if (LifecycleManagerDelegateControl.IsValid())
	{
		if (IMediaPlayerLifecycleManagerDelegate* Delegate = MediaModule->GetPlayerLifecycleManagerDelegate())
		{
			Delegate->OnMediaPlayerResourcesReleased(LifecycleManagerDelegateControl, ResourceFlags);
			return true;
		}
	}
	return false;
}

// ----------------------------------------------------------------------------------------------------------------------------------------------

void FMediaPlayerFacade::DestroyPlayer()
{
	FScopeLock Lock(&CriticalSection);

	if (!Player.IsValid())
	{
		return;
	}

	Player.Reset();
	NotifyLifetimeManagerDelegate_PlayerDestroyed();
	if (!PlayerUsesResourceReleaseNotification)
	{
		NotifyLifetimeManagerDelegate_PlayerResourcesReleased(IMediaPlayerLifecycleManagerDelegate::ResourceFlags_All);
	}
}

bool FMediaPlayerFacade::Open(const FString& Url, const IMediaOptions* Options, const FMediaPlayerOptions* PlayerOptions)
{
	SCOPE_CYCLE_COUNTER(STAT_MediaUtils_FacadeOpen);

	ActivePlayerOptions.Reset();

	if (IsRunningDedicatedServer())
	{
		return false;
	}

	Close();

	if (Url.IsEmpty())
	{
		return false;
	}

	check(MediaModule);

	// find a player factory for the intended playback
	IMediaPlayerFactory* PlayerFactory = GetPlayerFactoryForUrl(Url, Options);
	if (PlayerFactory == nullptr)
	{
		return false;
	}

	IMediaPlayerFactory* OldFactory(Player.IsValid() ? MediaModule->GetPlayerFactory(Player->GetPlayerPluginGUID()) : nullptr);

	bool bWillCreatePlayer = (!Player.IsValid() || PlayerFactory != OldFactory);
	uint64 NewPlayerInstanceID;
	uint32 WillUseNewResources;

	if (bWillCreatePlayer)
	{
		NewPlayerInstanceID = MediaModule->CreateMediaPlayerInstanceID();
		WillUseNewResources = IMediaPlayerLifecycleManagerDelegate::ResourceFlags_All; // as we create a new player we assume all resources a newly created in any case
	}
	else
	{
		check(Player.IsValid());
		NewPlayerInstanceID = PlayerInstanceID;
		WillUseNewResources = Player->GetNewResourcesOnOpen(); // ask player what resources it will create again even if it already exists
	}

	IMediaPlayerLifecycleManagerDelegate::IControlRef NewLifecycleManagerDelegateControl;
	if (FMediaPlayerFacade::NotifyLifetimeManagerDelegate_PlayerOpen(NewLifecycleManagerDelegateControl, Url, Options, PlayerOptions, PlayerFactory, bWillCreatePlayer, WillUseNewResources, NewPlayerInstanceID))
	{
		// Assume all is well: the delegate will either (have) submit(ted) the request or not -- in any case we need to assume the best -> "true"
		return true;
	}

	// We did not notify successfully or the delegate will not submit the request in its own. Do so here...
	return ContinueOpen(NewLifecycleManagerDelegateControl, Url, Options, PlayerOptions, PlayerFactory, Player, bWillCreatePlayer, NewPlayerInstanceID);
}

bool FMediaPlayerFacade::ContinueOpen(IMediaPlayerLifecycleManagerDelegate::IControlRef NewLifecycleManagerDelegateControl, const FString& Url, const IMediaOptions* Options, const FMediaPlayerOptions* PlayerOptions, IMediaPlayerFactory* PlayerFactory, TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> ReusedPlayer, bool bCreateNewPlayer, uint64 NewPlayerInstanceID)
{
	// Create or reuse player
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> NewPlayer(bCreateNewPlayer ? PlayerFactory->CreatePlayer(*this) : ReusedPlayer);

	// Continue initialization ---------------------------------------

	if (NewPlayer != Player)
	{
		DestroyPlayer();

		class FAsyncResourceReleaseNotification : public IMediaPlayer::IAsyncResourceReleaseNotification
		{
		public:
			FAsyncResourceReleaseNotification(IMediaPlayerLifecycleManagerDelegate::IControlRef InDelegateControl) : DelegateControl(InDelegateControl) {}

			virtual void Signal(uint32 ResourceFlags) override
			{
				TFunction<void()> NotifyTask = [TargetDelegateControl = DelegateControl, ResourceFlags]()
				{
					// Get MediaModule & check if it is already unloaded...
					IMediaModule* TargetMediaModule = FModuleManager::GetModulePtr<IMediaModule>("Media");
					if (TargetMediaModule)
					{
						// Delegate still there?
						if (IMediaPlayerLifecycleManagerDelegate* Delegate = TargetMediaModule->GetPlayerLifecycleManagerDelegate())
						{
							// Notify it!
							Delegate->OnMediaPlayerResourcesReleased(TargetDelegateControl, ResourceFlags);
						}
					}
				};
				Async(EAsyncExecution::TaskGraphMainThread, NotifyTask);
			};

			IMediaModule* MediaModule;
			IMediaPlayerLifecycleManagerDelegate::IControlRef DelegateControl;
		};

		FScopeLock Lock(&CriticalSection);
		Player = NewPlayer;
		PlayerInstanceID = NewPlayerInstanceID;
		LifecycleManagerDelegateControl = NewLifecycleManagerDelegateControl;
		PlayerUsesResourceReleaseNotification = LifecycleManagerDelegateControl.IsValid() ? Player->SetAsyncResourceReleaseNotification(TSharedRef<IMediaPlayer::IAsyncResourceReleaseNotification, ESPMode::ThreadSafe>(new FAsyncResourceReleaseNotification(LifecycleManagerDelegateControl))) : false;
	}
	else
	{
		LifecycleManagerDelegateControl = NewLifecycleManagerDelegateControl;
	}

	bool bIsRequestInvalid = (Player == nullptr);

#if !UE_BUILD_SHIPPING
	bIsRequestInvalid = bIsRequestInvalid || UE::MediaUtils::Private::CVarTestForcePlayerCreateFailed.GetValueOnAnyThread();
#endif

	if (bIsRequestInvalid)
	{
		NotifyLifetimeManagerDelegate_PlayerCreateFailed();
		// Make sure we don't get called from the "tickable" thread anymore - no need as we have no player
		MediaModule->GetTicker().RemoveTickable(AsShared());
		return false;
	}

	// Make sure we get ticked on the "tickable" thread
	// (this will not re-add us, should we already be registered)
	MediaModule->GetTicker().AddTickable(AsShared());

	// update the Guid
	Player->SetGuid(PlayerGuid);

	CurrentUrl = Url;

	if (PlayerOptions)
	{
		ActivePlayerOptions = *PlayerOptions;
	}

	// open the new media source
	if (!Player->Open(Url, Options, PlayerOptions))
	{
		NotifyLifetimeManagerDelegate_PlayerCreateFailed();
		CurrentUrl.Empty();
		ActivePlayerOptions.Reset();

		return false;
	}

#if WITH_EDITOR
	// Record the successful open to analytics
	if (Player->ShouldAutoRecordSourceOpenedEvent())
	{
		Player->RecordSourceOpenedEvent();
	}
#endif

	{
		FScopeLock Lock(&TimeValueLock);

		BlockOnRangeDisabled = false;
		BlockOnRange.OnFlush();
		LastVideoSampleProcessedTimeRange = TRange<FMediaTimeStamp>::Empty();
		LastAudioSampleProcessedTime.Invalidate();
		CurrentFrameVideoTimeStamp.Invalidate();
		CurrentFrameVideoDisplayTimeStamp.Invalidate();
		CurrentFrameAudioTimeStamp.Invalidate();

		NextEstVideoTimeAtFrameStart.Invalidate();
		SeekTargetTime.Invalidate();
		NextSeekTime.Reset();
		NextSequenceIndex.Reset();
		if (Player->GetPlayerFeatureFlag(IMediaPlayer::EFeatureFlag::UsePlaybackTimingV2))
		{
			NextSequenceIndex = 0;
		}
	}

	ResetTracks();

	if (bCreateNewPlayer)
	{
		NotifyLifetimeManagerDelegate_PlayerCreated();
	}

	return true;
}


void FMediaPlayerFacade::QueryCacheState(EMediaTrackType TrackType, EMediaCacheState State, TRangeSet<FTimespan>& OutTimeRanges) const
{
	if (!Player.IsValid())
	{
		return;
	}

	if (State == EMediaCacheState::Cached)
	{
		if (TrackType == EMediaTrackType::Audio)
		{
			Cache->GetCachedAudioSampleRanges(OutTimeRanges);
		}
		else if (TrackType == EMediaTrackType::Video)
		{
			Cache->GetCachedVideoSampleRanges(OutTimeRanges);
		}
	}
	else
	{
		if (TrackType == EMediaTrackType::Video)
		{
			Player->GetCache().QueryCacheState(State, OutTimeRanges);
		}
	}
}


bool FMediaPlayerFacade::Seek(const FTimespan& InTime)
{
	TimeValueLock.Lock();
	NextSeekTime.Reset();
	TimeValueLock.Unlock();

	auto CurrentPlayer = Player;

	if (!CurrentPlayer.IsValid())
	{
		return false;
	}

	FTimespan Duration = CurrentPlayer->GetControls().GetDuration();

	FTimespan Time;
	if (IsDurationValidAndFinite(Duration))
	{
		const TRange<FTimespan> ActiveRange = GetActivePlaybackRange();

		if (CurrentPlayer->GetControls().IsLooping())
		{
			const FTimespan ActiveRangeDuration = ActiveRange.GetUpperBoundValue() - ActiveRange.GetLowerBoundValue();
			Time = WrappedModulo(InTime - ActiveRange.GetLowerBoundValue(), ActiveRangeDuration) + ActiveRange.GetLowerBoundValue();
		}
		else
		{
			Time = FTimespan(FMath::Clamp(InTime.GetTicks(), ActiveRange.GetLowerBoundValue().GetTicks(), ActiveRange.GetUpperBoundValue().GetTicks()));
		}
	}
	else
	{
		Time = InTime;
	}

	FMediaSeekParams SeekParams;
	// v2 timing players are *required* to use the new sequence index we set up.
	TOptional<int32> SequenceIndexNow = NextSequenceIndex;
	if (CurrentPlayer->GetPlayerFeatureFlag(IMediaPlayer::EFeatureFlag::UsePlaybackTimingV2))
	{
		SeekParams.NewSequenceIndex = NextSequenceIndex = NextSequenceIndex.Get(0) + 1;
	}
	// Issue the seek.
	if (!CurrentPlayer->GetControls().Seek(Time, SeekParams))
	{
		// If that failed restore the sequence index.
		NextSequenceIndex = SequenceIndexNow;
		return false;
	}

	{
	FScopeLock Lock(&CriticalSection);
	// V2 timing player?
	if (CurrentPlayer->GetPlayerFeatureFlag(IMediaPlayer::EFeatureFlag::UsePlaybackTimingV2))
	{
		// Yes. Flush only the facade side of the system as needed for seeks
		// (the player is expected to flush its internal queues as needed itself)
		PrepareSampleQueueForSequenceIndex();
		Flush(CurrentPlayer->GetPlayerFeatureFlag(IMediaPlayer::EFeatureFlag::PlayerUsesInternalFlushOnSeek), true);
	}
	else
	{
		// No. Flush as requested...
		if (CurrentPlayer->FlushOnSeekStarted())
		{
			Flush(CurrentPlayer->GetPlayerFeatureFlag(IMediaPlayer::EFeatureFlag::PlayerUsesInternalFlushOnSeek), false);
		}
	}
	}

	TimeValueLock.Lock();
	SeekTargetTime = FMediaTimeStamp(Time, NextSequenceIndex.Get(0), 0);
	TimeValueLock.Unlock();
	return true;
}

bool FMediaPlayerFacade::IsSeeking() const
{
	FScopeLock Lock(&TimeValueLock);
	return SeekTargetTime.IsValid();
}

FMediaTimeStamp FMediaPlayerFacade::GetSeekTarget() const
{
	FScopeLock Lock(&TimeValueLock);
	return SeekTargetTime;
}

void FMediaPlayerFacade::SetNextSeek(const FTimespan& InTime)
{
	FScopeLock Lock(&TimeValueLock);
	NextSeekTime = InTime;
}

void FMediaPlayerFacade::SetBlockOnTime(const FTimespan& Time)
{
#if !MEDIAPLAYERFACADE_DISABLE_BLOCKING
	if (!Player.IsValid() || !Player->GetControls().CanControl(EMediaControl::BlockOnFetch))
	{
		return;
	}

	if (Time == FTimespan::MinValue())
	{
		BlockOnRange.SetRange(TRange<FTimespan>::Empty());
		Player->GetControls().SetBlockingPlaybackHint(false);
	}
	else
	{
		TRange<FTimespan> Range = TRange<FTimespan>::Inclusive(Time, Time);
		BlockOnRange.SetRange(Range);
		Player->GetControls().SetBlockingPlaybackHint(true);
	}
#endif
}


void FMediaPlayerFacade::SetBlockOnTimeRange(const TRange<FTimespan>& TimeRange)
{
#if !MEDIAPLAYERFACADE_DISABLE_BLOCKING
	BlockOnRange.SetRange(TimeRange);
#endif
}


void FMediaPlayerFacade::FBlockOnRange::OnFlush()
{
	LastProcessedTimeRange = TRange<FTimespan>::Empty();
	OnBlockSequenceIndex = 0;
	OnBlockLoopIndexOffset = 0;
	RangeIsDirty = true;
}


void FMediaPlayerFacade::FBlockOnRange::OnSeek(int32 PrimaryIndex)
{
	LastProcessedTimeRange = TRange<FTimespan>::Empty();
	OnBlockSequenceIndex = PrimaryIndex;
	OnBlockLoopIndexOffset = 0;
	RangeIsDirty = true;
}


void FMediaPlayerFacade::FBlockOnRange::SetRange(const TRange<FTimespan>& NewRange)
{
	if (CurrentTimeRange != NewRange)
	{
		if (NewRange.HasLowerBound() && NewRange.HasUpperBound() && NewRange.GetLowerBoundValue().GetTicks() <= NewRange.GetUpperBoundValue().GetTicks())
		{
			CurrentTimeRange = NewRange;
			RangeIsDirty = true;
		}
		else
		{
			Reset();
		}
	}
}


bool FMediaPlayerFacade::FBlockOnRange::IsSet() const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Facade->Player);

	if (!RangeIsDirty)
	{
		return !BlockOnRange.IsEmpty();
	}
	return !CurrentTimeRange.IsEmpty() && CurrentPlayer.IsValid() && CurrentPlayer->GetControls().CanControl(EMediaControl::BlockOnFetch);
}


const TRange<FMediaTimeStamp>& FMediaPlayerFacade::FBlockOnRange::GetRange() const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Facade->Player);

	if (!RangeIsDirty || !CurrentPlayer.IsValid())
	{
		return BlockOnRange;
	}

	// If the range is empty or the player can't support blocked playback or the range interval is invalid: reset everything & return empty block range...
	if (CurrentTimeRange.IsEmpty() || !CurrentPlayer->GetControls().CanControl(EMediaControl::BlockOnFetch) ||
		CurrentTimeRange.GetLowerBoundValue().GetTicks() > CurrentTimeRange.GetUpperBoundValue().GetTicks() ||
		CurrentPlayer->GetControls().GetDuration().GetTicks() <= 0)
	{
		LastProcessedTimeRange = TRange<FTimespan>::Empty();
		BlockOnRange = TRange<FMediaTimeStamp>::Empty();
		CurrentPlayer->GetControls().SetBlockingPlaybackHint(false);
		return BlockOnRange;
	}

	EMediaState PlayerState = CurrentPlayer->GetControls().GetState();
	if (PlayerState != EMediaState::Paused && PlayerState != EMediaState::Playing)
	{
		// Return an empty range. Note that the "IsSet()" method will still report a set block - so all code will remain in "external clock" mode,
		// but no samples will be requested (and no actual blocking should take place)
		static auto EmptyRange(TRange<FMediaTimeStamp>::Empty());
		return EmptyRange;
	}

	const int64 StartTicks = CurrentTimeRange.GetLowerBoundValue().GetTicks();
	const int64 EndExclusiveTicks = CurrentTimeRange.GetUpperBoundValue().GetTicks();
	const int64 DurationTicks = CurrentPlayer->GetControls().GetDuration().GetTicks();

	/*
		When looping we need to synthesize the expected start and end loop indices of the range we are returning.
		This is because on playback start/seeking the media player implicitly starts with a loop index of 0,
		while we could be in any of the looping repetitions in the sequencer track of this movie (if the track
		has been pulled out "to the right" to have it repeat the clip n times).
		Because of that we need to "lock" an initial loop offset that represents this initial difference and
		needs to be adjusted with any looping of the sequencer.
		Please note that this does not include the case where the media track ends before the end of the
		sequencer. In that case the media player is closed and re-opened on the media track boundaries.
	*/
	if (!CurrentPlayer->GetControls().IsLooping())
	{
		FTimespan Start(CurrentTimeRange.GetLowerBoundValue());
		FTimespan End(CurrentTimeRange.GetUpperBoundValue());
		const int32 LoopIdx = LastProcessedTimeRange.IsEmpty() ? 0 : LastProcessedTimeRange.GetLowerBoundValue().GetTicks() / DurationTicks;
		BlockOnRange = TRange<FMediaTimeStamp>(FMediaTimeStamp(Start, OnBlockSequenceIndex, OnBlockLoopIndexOffset + LoopIdx), FMediaTimeStamp(End, OnBlockSequenceIndex, OnBlockLoopIndexOffset + LoopIdx));
	}
	else
	{
		/*
			If this would be called very early in the player's startup after open() we would not yet be known... that would be fatal
				Should this actually happen in real-life applications, we could move the computations here into an accessor method used internally, so that this would be done
				only if data is processed, which would also mean: we know the duration!
				(Exception: live playback! --> but we would not allow blocking there anyway! (makes no sense as real life use case))
		*/
		if (!IsDurationValidAndFinite(CurrentPlayer->GetControls().GetDuration()))
		{
			// Catch if this is called to early and reset blocking...
			BlockOnRange = TRange<FMediaTimeStamp>::Empty();
			CurrentPlayer->GetControls().SetBlockingPlaybackHint(false);
			return BlockOnRange;
		}

		FMediaTimeStamp t0, t1;
		t0.SetTime(FTimespan(StartTicks % DurationTicks)).SetSequenceIndex(OnBlockSequenceIndex).SetLoopIndex(StartTicks / DurationTicks);
		t1.SetTime(FTimespan(EndExclusiveTicks% DurationTicks)).SetSequenceIndex(OnBlockSequenceIndex).SetLoopIndex(EndExclusiveTicks / DurationTicks);
		const bool bReverse = (Facade->GetUnpausedRate() < 0.0f);
		// Did we process a time range before?
		if (LastProcessedTimeRange.IsEmpty())
		{
			/*
				No, playback has just started fresh or through a seek.
				The media player will start with a loop index of 0, but the blocking range could be anywhere within
				a movie repetition (ie when the movie has been pulled out in the sequencer track to repeat a number of times).
				We set that repetion count as the base for the loop index.
			*/
			OnBlockLoopIndexOffset = !bReverse ? -t0.GetLoopIndex() : -t1.GetLoopIndex();
		}
		else
		{
			/*
				We already processed a time range. We now need to check if the current one has wrapped around in the
				current playback direction.

				Theoretically, with either very, very short movies or an excessively huge delta time this could have
				wrapped around more than once. We cannot detect this and hope this will not occur.
			*/
			if (!bReverse)
			{
				if (LastProcessedTimeRange.GetLowerBoundValue() > CurrentTimeRange.GetLowerBoundValue())
				{
					// Figure the loop index of the last range's start time.
					const int32 LastRangeLoopIdx = LastProcessedTimeRange.GetLowerBoundValue().GetTicks() / DurationTicks;
					OnBlockLoopIndexOffset += LastRangeLoopIdx + 1;
				}
			}
			else
			{
				if (LastProcessedTimeRange.GetLowerBoundValue() < CurrentTimeRange.GetLowerBoundValue())
				{
					// Figure the loop index of the last range's start time.
					const int32 ThisRangeLoopIdx = t0.GetLoopIndex();
					OnBlockLoopIndexOffset -= ThisRangeLoopIdx + 1;
				}
			}
		}

		// Assemble final blocking range
		t0.AdjustLoopIndex(OnBlockLoopIndexOffset);
		t1.AdjustLoopIndex(OnBlockLoopIndexOffset);
		BlockOnRange = TRange<FMediaTimeStamp>(t0, t1);
		check(!BlockOnRange.IsEmpty());
	}


	// Note: Due to varying DTs the new range will NOT be a simple monotone progression in playback direction, but might overlap or even be a subset of the previous one
	//		 We do not put any safeguards in place here, but rather use the "is last sample still valid" logic to reject illogical / impossible range requests.
	//		 All that aside: we DO expect ranges start (lower bound if forward, upper if reverse playback) to be moving in a monotone manner according to the set playback direction.
	CurrentPlayer->GetControls().SetBlockingPlaybackHint(!BlockOnRange.IsEmpty());
	LastProcessedTimeRange = CurrentTimeRange;
	RangeIsDirty = false;
	return BlockOnRange;
}


void FMediaPlayerFacade::SetCacheWindow(FTimespan Ahead, FTimespan Behind)
{
	Cache->SetCacheWindow(Ahead, Behind);
}


void FMediaPlayerFacade::SetGuid(FGuid& Guid)
{
	PlayerGuid = Guid;
}


bool FMediaPlayerFacade::SetLooping(bool Looping)
{
	return Player.IsValid() && Player->GetControls().SetLooping(Looping);
}


void FMediaPlayerFacade::SetMediaOptions(const IMediaOptions* Options)
{
}


bool FMediaPlayerFacade::SetRate(float Rate)
{
	// Enter CS as we change the rate which we read on the tickable thread
	FScopeLock Lock(&CriticalSection);

	if (!Player.IsValid())
	{
		return false;
	}

	// Is this new rate supported?
	bool bRateOk = true;
	if (Rate != 0.0f && !(Player->GetControls().GetSupportedRates(EMediaRateThinning::Thinned).Contains(Rate) || Player->GetControls().GetSupportedRates(EMediaRateThinning::Unthinned).Contains(Rate)))
	{
		// Pause player instead...
		// (some players may do this as a reaction to the illegal rate anyways - but we need to track the state properly!)
		Rate = 0.0f;
		bRateOk = false;
	}

	// Attempt to set the rate...
	if (!Player->GetControls().SetRate(Rate))
	{
		return false;
	}


	// Any change?
	if (CurrentRate == Rate)
	{
		// no change - just return with ok status
		return bRateOk;
	}

	// Notify sinks of rate change
	FMediaSampleSinkEventData Data;
	Data.PlaybackRateChanged.PlaybackRate = Rate;
	SendSinkEvent(EMediaSampleSinkEvent::PlaybackRateChanged, Data);

	if ((LastRate * Rate) < 0.0f)
	{
		// direction change
		Flush(Player->GetPlayerFeatureFlag(IMediaPlayer::EFeatureFlag::UsePlaybackTimingV2), false);
	}
	else
	{
		if (Rate == 0.0f)
		{
			// Invalidate audio time on entering pause mode...
			if (TSharedPtr<FMediaAudioSampleSink, ESPMode::ThreadSafe> AudioSink = PrimaryAudioSink.Pin())
			{
				AudioSink->InvalidateAudioTime();
			}
		}
	}

	// Track last "unpaused" rate we set
	if (Rate != 0.0)
	{
		LastRate = Rate;
	}
	CurrentRate = Rate;

	return bRateOk;
}


bool FMediaPlayerFacade::SetNativeVolume(float Volume)
{
	return Player.IsValid() ? Player->SetNativeVolume(Volume) : false;
}


bool FMediaPlayerFacade::SetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex, int32 FormatIndex)
{
	return Player.IsValid() ? Player->GetTracks().SetTrackFormat((EMediaTrackType)TrackType, TrackIndex, FormatIndex) : false;
}


bool FMediaPlayerFacade::SetVideoTrackFrameRate(int32 TrackIndex, int32 FormatIndex, float FrameRate)
{
	return Player.IsValid() ? Player->GetTracks().SetVideoTrackFrameRate(TrackIndex, FormatIndex, FrameRate) : false;
}


bool FMediaPlayerFacade::SetViewField(float Horizontal, float Vertical, bool Absolute)
{
	return Player.IsValid() && Player->GetView().SetViewField(Horizontal, Vertical, Absolute);
}


bool FMediaPlayerFacade::SetViewOrientation(const FQuat& Orientation, bool Absolute)
{
	return Player.IsValid() && Player->GetView().SetViewOrientation(Orientation, Absolute);
}


bool FMediaPlayerFacade::SupportsRate(float Rate, bool Unthinned) const
{
	EMediaRateThinning Thinning = Unthinned ? EMediaRateThinning::Unthinned : EMediaRateThinning::Thinned;
	return Player.IsValid() && Player->GetControls().GetSupportedRates(Thinning).Contains(Rate);
}

TRange<FTimespan> FMediaPlayerFacade::GetPlaybackTimeRange(EMediaTimeRangeType InRangeToGet) const
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	return CurrentPlayer.IsValid() ? CurrentPlayer->GetControls().GetPlaybackTimeRange(InRangeToGet) : TRange<FTimespan>();
}

bool FMediaPlayerFacade::SetPlaybackTimeRange(const TRange<FTimespan>& InTimeRange)
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	return CurrentPlayer.IsValid() ? CurrentPlayer->GetControls().SetPlaybackTimeRange(InTimeRange) : false;
}


void FMediaPlayerFacade::SetLastAudioRenderedSampleTime(FTimespan SampleTime)
{
	FScopeLock Lock(&TimeValueLock);
	LastAudioRenderedSampleTime.TimeStamp = FMediaTimeStamp(SampleTime);
	LastAudioRenderedSampleTime.SampledAtTime = FPlatformTime::Seconds();
}

FTimespan FMediaPlayerFacade::GetLastAudioRenderedSampleTime() const
{
	FScopeLock Lock(&TimeValueLock);
	return LastAudioRenderedSampleTime.TimeStamp.Time;
}

void FMediaPlayerFacade::SetAreEventsSafeForAnyThread(bool bInAreEventsSafeForAnyThread)
{
	bAreEventsSafeForAnyThread = bInAreEventsSafeForAnyThread;
}

/* FMediaPlayerFacade implementation
*****************************************************************************/

bool FMediaPlayerFacade::BlockOnFetch() const
{
	check(Player.IsValid());

	const TRange<FMediaTimeStamp> BR(GetAdjustedBlockOnRange());

	if (BR.IsEmpty() || !Player->GetControls().CanControl(EMediaControl::BlockOnFetch) || BlockOnRangeDisabled || bHaveActiveAudio)
	{
		return false; // no blocking requested / not supported / audio present
	}

	if (Player->GetPlayerFeatureFlag(IMediaPlayer::EFeatureFlag::UsePlaybackTimingV2))
	{
		//
		// V2 blocking logic
		//

		// note: with V2 timing we only get here if any current sample is no longer considered "valid" and we didn't so far get a new one that would be
		//  -->  we do not need to check the actual range here; we only check for exceptions, where we can proceed although we don't have the sample...

		// The next checks make only sense if the player is done preparing...
		if (!IsPreparing())
		{
			// Looping off?
			if (!Player->GetControls().IsLooping())
			{
				// Yes. Is the sample outside the media's range?
				// (note: this assumes the media starts at time ZERO - this will not be the case at all times (e.g. live playback) -- for now we assume a player will flagged blocked playback as invalid in that case!)
				if (BR.GetUpperBoundValue().GetTime() < FTimespan::Zero() || Player->GetControls().GetDuration() <= BR.GetLowerBoundValue().GetTime())
				{
					return false;
				}
			}
		}

		// Block until sample arrives!
		return true;
	}
	else
	{
		//
		// V1 blocking logic
		//

		if (IsPreparing())
		{
			return true; // block on media opening
		}

		if (!IsPlaying())
		{
			// no blocking if we are not playing (e.g. paused)
			return false;
		}

		if (CurrentRate < 0.0f)
		{
			return false; // block only in forward play
		}

		const bool VideoReady = (VideoSampleSinks.Num() == 0) || (BR.GetUpperBoundValue().Time < NextVideoSampleTime);

		if (VideoReady)
		{
			return false; // video is ready
		}

		return true;
	}

}


void FMediaPlayerFacade::Flush(bool bExcludePlayer, bool bOnSeek)
{
	UE_LOGF(LogMediaUtils, Verbose, "PlayerFacade: Flushing sinks");

	FScopeLock Lock(&CriticalSection);

	auto RawMediaPlayer = MediaPlayer.Get();
	AudioSampleSinks.Flush(RawMediaPlayer);
	CaptionSampleSinks.Flush(RawMediaPlayer);
	MetadataSampleSinks.Flush(RawMediaPlayer);
	SubtitleSampleSinks.Flush(RawMediaPlayer);
	VideoSampleSinks.Flush(RawMediaPlayer);
	MostRecentlyDeliveredVideoFrameTimecode.Reset();

	if (Player.IsValid() && !bExcludePlayer)
	{
		Player->GetSamples().FlushSamples();
	}

	LastAudioRenderedSampleTime.Invalidate();
	if (bOnSeek)
	{
		BlockOnRange.OnSeek(NextSequenceIndex.Get(0));
	}
	else
	{
		BlockOnRange.OnFlush();
	}

	if (Player.IsValid() && Player->GetPlayerFeatureFlag(IMediaPlayer::EFeatureFlag::UsePlaybackTimingV2))
	{
		// Logically we have no old sample anymore if we did seek
		// (as in: we will start asking for a new one until we get one - even with a rate of zero, if we had a non-zero one ever before)
		if (bOnSeek)
		{
			LastVideoSampleProcessedTimeRange = TRange<FMediaTimeStamp>::Empty();
		}
		else
		{
			if (!bExcludePlayer && !LastVideoSampleProcessedTimeRange.IsEmpty())
			{
				// Players will reset their sequence index related values, but keep the playback position. Adjust our record accordingly...
				int32 LoopIdxS = LastVideoSampleProcessedTimeRange.GetLowerBoundValue().GetLoopIndex();
				int32 LoopIdxE = LastVideoSampleProcessedTimeRange.GetUpperBoundValue().GetLoopIndex();
				LastVideoSampleProcessedTimeRange.SetLowerBoundValue(FMediaTimeStamp(LastVideoSampleProcessedTimeRange.GetLowerBoundValue().Time, 0, 0));
				LastVideoSampleProcessedTimeRange.SetUpperBoundValue(FMediaTimeStamp(LastVideoSampleProcessedTimeRange.GetUpperBoundValue().Time, 0, LoopIdxE - LoopIdxS));
			}
		}

		// Invalidate next video time to fetch (non-audio case)
		NextEstVideoTimeAtFrameStart.Invalidate();
		// ...and seek target
		TimeValueLock.Lock();
		SeekTargetTime.Invalidate();
		TimeValueLock.Unlock();
	}

	NextVideoSampleTime = FTimespan::MinValue();
}


void FMediaPlayerFacade::SendSinkEvent(EMediaSampleSinkEvent Event, const FMediaSampleSinkEventData& Data)
{
	{
	FScopeLock Lock(&CriticalSection);

	AudioSampleSinks.ReceiveEvent(Event, Data);
	MetadataSampleSinks.ReceiveEvent(Event, Data);
	}

	CaptionSampleSinks.ReceiveEvent(Event, Data);
	SubtitleSampleSinks.ReceiveEvent(Event, Data);
	VideoSampleSinks.ReceiveEvent(Event, Data);
}


bool FMediaPlayerFacade::GetAudioTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaAudioTrackFormat& OutFormat) const
{
	if (TrackIndex == INDEX_NONE)
	{
		TrackIndex = GetSelectedTrack(EMediaTrackType::Audio);
	}

	if (FormatIndex == INDEX_NONE)
	{
		FormatIndex = GetTrackFormat(EMediaTrackType::Audio, TrackIndex);
	}

	return (Player.IsValid() && Player->GetTracks().GetAudioTrackFormat(TrackIndex, FormatIndex, OutFormat));
}


IMediaPlayerFactory* FMediaPlayerFacade::GetPlayerFactoryForUrl(const FString& Url, const IMediaOptions* Options) const
{
	FName PlayerName;

	if (DesiredPlayerName != NAME_None)
	{
		PlayerName = DesiredPlayerName;
	}
	else if (Options != nullptr)
	{
		PlayerName = Options->GetDesiredPlayerName();
	}
	else
	{
		PlayerName = NAME_None;
	}

	if (MediaModule == nullptr)
	{
		UE_LOGF(LogMediaUtils, Error, "Failed to load Media module");
		return nullptr;
	}

	//
	// Reuse existing player if explicitly requested name matches
	//
	if (Player.IsValid())
	{
		IMediaPlayerFactory* CurrentFactory = MediaModule->GetPlayerFactory(Player->GetPlayerPluginGUID());
		if (PlayerName == CurrentFactory->GetPlayerName())
		{
			return CurrentFactory;
		}
	}

	//
	// Try to create explicitly requested player
	//
	if (PlayerName != NAME_None)
	{
		IMediaPlayerFactory* Factory = MediaModule->GetPlayerFactory(PlayerName);

		if (Factory == nullptr)
		{
			UE_LOGF(LogMediaUtils, Error, "Could not find desired player %ls for %ls", *PlayerName.ToString(), *Url);
		}

		return Factory;
	}

	//
	// Try to find a fitting player with no explicit name given
	//
	const TArray<IMediaPlayerFactory*>& PlayerFactories = MediaModule->GetPlayerFactories();
	if (PlayerFactories.Num() == 0)
	{
		UE_LOGF(LogMediaUtils, Error, "Cannot play %ls: no media player plug-ins are installed and enabled in this project", *Url);
		return nullptr;
	}
	struct FCandidate
	{
		FName Name;
		IMediaPlayerFactory* Factory = nullptr;
		int32 ConfidenceScore = 0;
	};
	TArray<FCandidate> Candidates;
	const FString RunningPlatformName(FPlatformProperties::IniPlatformName());
	for(IMediaPlayerFactory* Factory : PlayerFactories)
	{
		if (Factory->SupportsPlatform(RunningPlatformName))
		{
			int32 ConfidenceScore = Factory->GetPlayabilityConfidenceScore(Url, Options, nullptr, nullptr);
			if (ConfidenceScore > 0)
			{
				FCandidate& Candidate = Candidates.Emplace_GetRef();
				Candidate.Name = Factory->GetPlayerName();
				Candidate.Factory = Factory;
				Candidate.ConfidenceScore = ConfidenceScore;
			}
		}
	}
	Candidates.Sort([](const FCandidate& c1, const FCandidate& c2)
	{
		// If both factories are equally confident, sort alphabetically by name.
		if (c1.ConfidenceScore == c2.ConfidenceScore)
		{
			return c1.Name.ToString() < c2.Name.ToString();
		}
		// Sort by descending confidence score.
		return c1.ConfidenceScore > c2.ConfidenceScore;
	});
	if (Candidates.Num())
	{
		return Candidates[0].Factory;
	}
	//
	// No suitable player found!
	//
	UE_LOGF(LogMediaUtils, Error, "Cannot play %ls, because none of the enabled media player plug-ins support it:", *Url);
	for (IMediaPlayerFactory* Factory : PlayerFactories)
	{
		if (Factory->SupportsPlatform(RunningPlatformName))
		{
			UE_LOGF(LogMediaUtils, Log, "| %ls (URI scheme or file extension not supported)", *Factory->GetPlayerName().ToString());
		}
		else
		{
			UE_LOGF(LogMediaUtils, Log, "| %ls (only available on %ls, but not on %ls)", *Factory->GetPlayerName().ToString(), *FString::Join(Factory->GetSupportedPlatforms(), TEXT(", ")), *RunningPlatformName);
		}
	}
	return nullptr;
}


bool FMediaPlayerFacade::GetVideoTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaVideoTrackFormat& OutFormat) const
{
	if (TrackIndex == INDEX_NONE)
	{
		TrackIndex = GetSelectedTrack(EMediaTrackType::Video);
	}

	if (FormatIndex == INDEX_NONE)
	{
		FormatIndex = GetTrackFormat(EMediaTrackType::Video, TrackIndex);
	}

	return (Player.IsValid() && Player->GetTracks().GetVideoTrackFormat(TrackIndex, FormatIndex, OutFormat));
}


void FMediaPlayerFacade::ProcessEvent(EMediaEvent Event, bool bIsBroadcastAllowed)
{
	SCOPE_CYCLE_COUNTER(STAT_MediaUtils_FacadeProcessEvent);

	if ((Event == EMediaEvent::MediaOpened) || (Event == EMediaEvent::MediaOpenFailed))
	{
		if (Event == EMediaEvent::MediaOpenFailed)
		{
			CurrentUrl.Empty();
		}

		const FString MediaInfo = Player.IsValid() ? Player->GetInfo() : TEXT("");

		if (MediaInfo.IsEmpty())
		{
			UE_LOGF(LogMediaUtils, Verbose, "PlayerFacade: Media Info: n/a");
		}
		else
		{
			UE_LOGF(LogMediaUtils, Verbose, "PlayerFacade: Media Info:\n%ls", *MediaInfo);
		}
	}
	else if (Event == EMediaEvent::TracksChanged)
	{
		SelectDefaultTracks();
		if (Player.IsValid())
		{
			// Apply track selection immediately so the selection can be queried.
			UpdateTrackSelectionWithPlayer();
			if (!Player->GetPlayerFeatureFlag(IMediaPlayer::EFeatureFlag::UsePlaybackTimingV2))
			{
				// Execute flush for older players only
				Flush();
			}
		}
	}
	else if (Event == EMediaEvent::SeekCompleted)
	{
		// We only consider flushing on seek completion if there is a V1 timing player...
		if (Player.IsValid() && !Player->GetPlayerFeatureFlag(IMediaPlayer::EFeatureFlag::UsePlaybackTimingV2))
		{
			// Does the player want this?
			if (Player->FlushOnSeekCompleted())
			{
				Flush(Player->GetPlayerFeatureFlag(IMediaPlayer::EFeatureFlag::PlayerUsesInternalFlushOnSeek), true);
			}
		}
	}
	else if (Event == EMediaEvent::MediaClosed)
	{
		// Player still closed?
		if (CurrentUrl.IsEmpty())
		{
			// Yes, this also means: if we still have a player, it's still the one this event originated from
			FMediaSampleSinkEventData Data;
			SendSinkEvent(EMediaSampleSinkEvent::MediaClosed, Data);

			// If player allows: close it down all the way right now
			if (Player.IsValid() && Player->GetPlayerFeatureFlag(IMediaPlayer::EFeatureFlag::AllowShutdownOnClose))
			{
				bDidRecentPlayerHaveError = HasError();
				DestroyPlayer();
			}

			// Stop issuing audio thread ticks until we open the player again
			MediaModule->GetTicker().RemoveTickable(AsShared());
		}
	}
	else if (Event == EMediaEvent::PlaybackEndReached)
	{
		if (Player.IsValid() && !Player->GetPlayerFeatureFlag(IMediaPlayer::EFeatureFlag::UsePlaybackTimingV2))
		{
			// Execute flush for older players only
			Flush();
		}
		FMediaSampleSinkEventData Data;
		SendSinkEvent(EMediaSampleSinkEvent::PlaybackEndReached, Data);
	}

	if (bIsBroadcastAllowed)
	{
		MediaEvent.Broadcast(Event);
	}
	else
	{
		QueuedEventBroadcasts.Enqueue(Event);
	}
}


void FMediaPlayerFacade::ResetTracks()
{
	for (int32 Idx = 0; Idx < (int32)EMediaTrackType::Num; ++Idx)
	{
		TrackSelection.UserSelection[Idx] = -1;
		TrackSelection.PlayerSelection[Idx] = -1;
	}
}


void FMediaPlayerFacade::SelectDefaultTracks()
{
	// See if the player has selected appropriate default tracks.
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (CurrentPlayer.IsValid() && CurrentPlayer->GetPlayerFeatureFlag(IMediaPlayer::EFeatureFlag::PlayerSelectsDefaultTracks))
	{
		ResetTracks();
		// Get what the player has selected as user defaults.
		// The TrackSelection.PlayerSelection[...] will be updated in UpdateTrackSelectionWithPlayer()
		// where the existence of sinks is checked for.
		IMediaTracks& Tracks = CurrentPlayer->GetTracks();
		for(int32 Idx=0; Idx<(int32)EMediaTrackType::Num; ++Idx)
		{
			TrackSelection.UserSelection[Idx] = Tracks.GetSelectedTrack((EMediaTrackType)Idx);
		}
		// If overrides are set, use them.
		if (ActivePlayerOptions.IsSet())
		{
			if (ActivePlayerOptions.GetValue().TrackSelection == EMediaPlayerOptionTrackSelectMode::UseTrackOptionIndices)
			{
				FMediaPlayerTrackOptions TrackOptions;
				TrackOptions = ActivePlayerOptions.GetValue().Tracks;
				TrackSelection.UserSelection[(int32)EMediaTrackType::Audio] = TrackOptions.Audio;
				TrackSelection.UserSelection[(int32)EMediaTrackType::Caption] = TrackOptions.Caption;
				TrackSelection.UserSelection[(int32)EMediaTrackType::Metadata] = TrackOptions.Metadata;
				TrackSelection.UserSelection[(int32)EMediaTrackType::Subtitle] = TrackOptions.Subtitle;
				TrackSelection.UserSelection[(int32)EMediaTrackType::Video] = TrackOptions.Video;
			}
		}
	}
	else
	{
		FMediaPlayerTrackOptions TrackOptions;
		if (ActivePlayerOptions.IsSet())
		{
			if (ActivePlayerOptions.GetValue().TrackSelection == EMediaPlayerOptionTrackSelectMode::UseTrackOptionIndices)
			{
				TrackOptions = ActivePlayerOptions.GetValue().Tracks;
			}
		}

		TrackSelection.UserSelection[(int32)EMediaTrackType::Audio] = TrackOptions.Audio;
		TrackSelection.UserSelection[(int32)EMediaTrackType::Caption] = TrackOptions.Caption;
		TrackSelection.UserSelection[(int32)EMediaTrackType::Metadata] = TrackOptions.Metadata;
		TrackSelection.UserSelection[(int32)EMediaTrackType::Subtitle] = TrackOptions.Subtitle;
		TrackSelection.UserSelection[(int32)EMediaTrackType::Video] = TrackOptions.Video;
	}
}


bool FMediaPlayerFacade::SelectTrack(EMediaTrackType TrackType, int32 TrackIndex)
{
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (CurrentPlayer.IsValid())
	{
		IMediaTracks& Tracks = CurrentPlayer->GetTracks();

		if (Tracks.GetNumTracks(TrackType) > TrackIndex)
		{
			TrackSelection.UserSelection[(int32)TrackType] = TrackIndex;
			return true;
		}
	}
	return false;
}


int32 FMediaPlayerFacade::GetSelectedTrack(EMediaTrackType TrackType) const
{
	return TrackSelection.UserSelection[(int32)TrackType];
}


void FMediaPlayerFacade::UpdateTrackSelectionWithPlayer()
{
	check(Player.IsValid());

	bool bChanges = false;

	IMediaTracks& Tracks = Player->GetTracks();
	for (int32 Idx = 0; Idx < (int32)EMediaTrackType::Num; ++Idx)
	{
		// Player and user selection are different?
		if (TrackSelection.PlayerSelection[Idx] != TrackSelection.UserSelection[Idx])
		{
			// Yes...
			int32 UserSelection = TrackSelection.UserSelection[Idx];

			// Filter selection against the configured sinks...
			if (UserSelection != -1)
			{
				if ((Idx == (int)EMediaTrackType::Audio && !PrimaryAudioSink.IsValid()) ||
					(Idx == (int)EMediaTrackType::Video && VideoSampleSinks.IsEmpty()) ||
					(Idx == (int)EMediaTrackType::Caption && CaptionSampleSinks.IsEmpty()) ||
					(Idx == (int)EMediaTrackType::Subtitle && SubtitleSampleSinks.IsEmpty()) ||
					(Idx == (int)EMediaTrackType::Metadata && MetadataSampleSinks.IsEmpty()))
				{
					UserSelection = -1;
				}
			}

			// After filtering the user's selection, do we still have to change things?
			if (TrackSelection.PlayerSelection[Idx] != UserSelection)
			{
				// Yes!
				if (Tracks.SelectTrack((EMediaTrackType)Idx, UserSelection))
				{
					// Recall what is now selected with the player...
					TrackSelection.PlayerSelection[Idx] = UserSelection;

					bChanges = true;
				}
				else
				{
					// Track selection failed. Patch the user selection to be what we know of the player's, so we do not reattempt this over and over...
					TrackSelection.UserSelection[Idx] = TrackSelection.PlayerSelection[Idx];
				}
			}
		}
	}

	if (bChanges)
	{
		if (!Player->GetPlayerFeatureFlag(IMediaPlayer::EFeatureFlag::IsTrackSwitchSeamless))
		{
			Flush();
		}
	}
}


float FMediaPlayerFacade::GetUnpausedRate() const
{
	return (CurrentRate == 0.0f) ? LastRate : CurrentRate;
}


/* IMediaClockSink interface
*****************************************************************************/

void FMediaPlayerFacade::TickInput(FTimespan DeltaTime, FTimespan Timecode)
{
	SCOPE_CYCLE_COUNTER(STAT_MediaUtils_FacadeTickInput);

	if (Player.IsValid())
	{
		UpdateTrackSelectionWithPlayer();
		MonitorAudioEnablement();

		Player->TickInput(DeltaTime, Timecode);

		bool bIsBroadcastAllowed = bAreEventsSafeForAnyThread || IsInGameThread();
		if (Player->GetPlayerFeatureFlag(IMediaPlayer::EFeatureFlag::UsePlaybackTimingV2))
		{
			//
			// New timing control (handled before any engine world, object etc. updates; so "all frame" (almost) see the state produced here)
			//

			// process deferred events
			// NOTE: if there is no player anymore we execute the remaining queued events in TickFetch (backwards compatibility - should move here once V1 support removed)
			EMediaEvent Event;
			if (bIsBroadcastAllowed)
			{
				while(QueuedEventBroadcasts.Dequeue(Event))
				{
					MediaEvent.Broadcast(Event);
				}
			}
			while(QueuedEvents.Dequeue(Event))
			{
				ProcessEvent(Event, bIsBroadcastAllowed);
			}

			// Handling events may have killed the player. Did it?
			if (!Player.IsValid())
			{
				// If so: nothing more to do!
				return;
			}

			//
			// Setup timing for sample processing
			//
			PreSampleProcessingTimeHandling();

			TRange<FMediaTimeStamp> TimeRange;
			if (!GetCurrentPlaybackTimeRange(TimeRange, CurrentRate, DeltaTime, false))
			{
				return;
			}

			SET_FLOAT_STAT(STAT_MediaUtils_FacadeTime, TimeRange.GetLowerBoundValue().Time.GetTotalSeconds());

			//
			// Process samples in range
			//
			IMediaSamples& Samples = Player->GetSamples();

			double BlockingStart = FPlatformTime::Seconds();
			while(1)
			{
				ProcessCaptionSamples(Samples, TimeRange);
				ProcessSubtitleSamples(Samples, TimeRange);

				if (ProcessVideoSamples(Samples, TimeRange))
				{
					// We either got a new sample or a current one is still the best choice...
					break;
				}

				// The current one is outdated and no new one was delivered. Should we block for one?
				if (!BlockOnFetch())
				{
					// No... continue...
					break;
				}

				// Issue tick call with dummy timing as some players advance some state in the tick, which we wait for
				Player->TickInput(FTimespan::Zero(), FTimespan::MinValue());

				// Monitor / update seek status
				UpdateSeekStatus();

				// Process deferred events & check for events that break the block
				bool bEventCancelsBlock = false;
				while(QueuedEvents.Dequeue(Event))
				{
					if (Event == EMediaEvent::MediaClosed || Event == EMediaEvent::MediaOpenFailed)
					{
						bEventCancelsBlock = true;
					}
					ProcessEvent(Event, bIsBroadcastAllowed);
				}

				// We might have lost the player during event handling or an event breaks the block...
				if (!Player.IsValid() || bEventCancelsBlock)
				{
					// Disable blocking feature for now (a new open would reset this)
					UE_LOGF(LogMediaUtils, Warning, "Blocking media playback closed or failed. Disabling it for this playback session.");
					BlockOnRangeDisabled = true;
					break;
				}

				// Timeout?
				if ((FPlatformTime::Seconds() - BlockingStart) > static_cast<double>(UE::MediaUtils::Private::GBlockOnFetchTimeout))
				{
					FString Url;
#if !UE_BUILD_SHIPPING
					Url = Player->GetUrl();
#endif // !UE_BUILD_SHIPPING
					UE_LOGF(LogMediaUtils, Error, "Blocking media playback timed out on range [%ls (seq:%d, loop:%d), %ls (seq:%d, loop:%d)]. Disabling it for this playback session. URL:%ls",
						*TimeRange.GetLowerBoundValue().Time.ToString(TEXT("%h:%m:%s.%t")), TimeRange.GetLowerBoundValue().GetSequenceIndex(), TimeRange.GetLowerBoundValue().GetLoopIndex(),
						*TimeRange.GetUpperBoundValue().Time.ToString(TEXT("%h:%m:%s.%t")), TimeRange.GetUpperBoundValue().GetSequenceIndex(), TimeRange.GetUpperBoundValue().GetLoopIndex(),
						*Url);
					BlockOnRangeDisabled = true;
					break;
				}

				FPlatformProcess::Sleep(0.0f);
			}

			SET_DWORD_STAT(STAT_MediaUtils_FacadeNumVideoSamples, Samples.NumVideoSamples());

			//
			// Advance timing etc.
			//
			PostSampleProcessingTimeHandling(DeltaTime);

			if (bHaveActiveAudio)
			{
				// Keep currently last processed audio sample timestamp available for all frame (to provide consistent info)
				FScopeLock Lock(&TimeValueLock);
				CurrentFrameAudioTimeStamp = LastAudioSampleProcessedTime.TimeStamp;
			}
		}

		// Check if primary audio sink needs a change and make sure invalid sinks are purged at all times
		PrimaryAudioSink = AudioSampleSinks.GetPrimaryAudioSink();
	}
}


void FMediaPlayerFacade::TickFetch(FTimespan DeltaTime, FTimespan Timecode)
{
	SCOPE_CYCLE_COUNTER(STAT_MediaUtils_FacadeTickFetch);

	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer(Player);
	if (!CurrentPlayer.IsValid())
	{
		// Send out deferred broadcasts.
		EMediaEvent Event;
		bool bIsBroadcastAllowed = bAreEventsSafeForAnyThread || IsInGameThread();
		if (bIsBroadcastAllowed)
		{
			while(QueuedEventBroadcasts.Dequeue(Event))
			{
				MediaEvent.Broadcast(Event);
			}
		}

		// process deferred events
		while(QueuedEvents.Dequeue(Event))
		{
			ProcessEvent(Event, bIsBroadcastAllowed);
		}
		return;
	}

	if (!CurrentPlayer->GetPlayerFeatureFlag(IMediaPlayer::EFeatureFlag::UsePlaybackTimingV2))
	{
		//
		// Old timing control
		//

		// let the player generate samples & process events
		CurrentPlayer->TickFetch(DeltaTime, Timecode);

		{
			// process deferred events
			EMediaEvent Event;
			while(QueuedEvents.Dequeue(Event))
			{
				ProcessEvent(Event, true);
			}
		}

		TRange<FTimespan> TimeRange;

		const FTimespan CurrentTime = GetTime();

		SET_FLOAT_STAT(STAT_MediaUtils_FacadeTime, CurrentTime.GetTotalSeconds());

		// get current play rate
		float Rate = GetUnpausedRate();

		if (Rate > 0.0f)
		{
			TimeRange = TRange<FTimespan>::AtMost(CurrentTime);
		}
		else if (Rate < 0.0f)
		{
			TimeRange = TRange<FTimespan>::AtLeast(CurrentTime);
		}
		else
		{
			TimeRange = TRange<FTimespan>(CurrentTime);
		}

		// process samples in range
		IMediaSamples& Samples = CurrentPlayer->GetSamples();

		const double BlockOnFetchTimeout = static_cast<double>(UE::MediaUtils::Private::GBlockOnFetchTimeout);
		bool Blocked = false;
		FDateTime BlockedTime;

		while(true)
		{
			ProcessCaptionSamplesV1(Samples, TimeRange);
			ProcessSubtitleSamplesV1(Samples, TimeRange);
			ProcessVideoSamplesV1(Samples, TimeRange);

			if (!BlockOnFetch())
			{
				break;
			}

			if (Blocked)
			{
				if ((FDateTime::UtcNow() - BlockedTime) >= FTimespan::FromSeconds(BlockOnFetchTimeout))
				{
					UE_LOGF(LogMediaUtils, Verbose, "PlayerFacade: Aborted block on fetch %ls after %i seconds",
						*BlockOnRange.GetRange().GetLowerBoundValue().Time.ToString(TEXT("%h:%m:%s.%t")),
						static_cast<int32>(BlockOnFetchTimeout)
					);

					break;
				}
			}
			else
			{
				UE_LOGF(LogMediaUtils, VeryVerbose, "PlayerFacade: Blocking on fetch %ls", *BlockOnRange.GetRange().GetLowerBoundValue().Time.ToString(TEXT("%h:%m:%s.%t")));

				Blocked = true;
				BlockedTime = FDateTime::UtcNow();
			}

			FPlatformProcess::Sleep(0.0f);
		}
	}
}


void FMediaPlayerFacade::TickOutput(FTimespan DeltaTime, FTimespan /*Timecode*/)
{
	SCOPE_CYCLE_COUNTER(STAT_MediaUtils_FacadeTickOutput);

	if (!Player.IsValid())
	{
		return;
	}

	Cache->Tick(DeltaTime, CurrentRate, GetTime());

	ExecuteNextSeek();
}


/* IMediaTickable interface
*****************************************************************************/

void FMediaPlayerFacade::TickTickable()
{
	SCOPE_CYCLE_COUNTER(STAT_MediaUtils_FacadeTickTickable);

	FScopeLock Lock(&CriticalSection);

	if (!Player.IsValid())
	{
		return;
	}

	float Rate = GetUnpausedRate();
	if (Rate == 0.0f)
	{
		return;
	}

	{
		FScopeLock Lock1(&TimeValueLock);
		Player->SetLastAudioRenderedSampleTime(LastAudioRenderedSampleTime.TimeStamp.Time);
	}

	Player->TickAudio();

	// determine range of valid samples

	// process samples in range
	IMediaSamples& Samples = Player->GetSamples();

	if (Player->GetPlayerFeatureFlag(IMediaPlayer::EFeatureFlag::UsePlaybackTimingV2))
	{
		ProcessAudioSamples(Samples, TRange<FMediaTimeStamp>());

		const FMediaTimeStamp Time = GetTimeStamp();
		auto TimeRange = TRange<FMediaTimeStamp>::Inclusive(FMediaTimeStamp(FTimespan::MinValue(), 0, 0), Time + MediaPlayerFacade::MetadataPreroll);
		ProcessMetadataSamples(Samples, TimeRange);
	}
	else
	{
		TRange<FTimespan> AudioTimeRange;
		TRange<FTimespan> MetadataTimeRange;

		const FTimespan Time = GetTime();

		if (Rate >= 0.0f)
		{
			AudioTimeRange = TRange<FTimespan>::Inclusive(FTimespan::MinValue(), Time + MediaPlayerFacade::AudioPreroll);
			MetadataTimeRange = TRange<FTimespan>::Inclusive(FTimespan::MinValue(), Time + MediaPlayerFacade::MetadataPreroll);
		}
		else
		{
			AudioTimeRange = TRange<FTimespan>::Inclusive(Time - MediaPlayerFacade::AudioPreroll, FTimespan::MaxValue());
			MetadataTimeRange = TRange<FTimespan>::Inclusive(Time - MediaPlayerFacade::MetadataPreroll, FTimespan::MaxValue());
		}

		ProcessAudioSamplesV1(Samples, AudioTimeRange);
		ProcessMetadataSamplesV1(Samples, MetadataTimeRange);
	}

	SET_DWORD_STAT(STAT_MediaUtils_FacadeNumAudioSamples, Samples.NumAudioSamples());
}


void FMediaPlayerFacade::PrepareSampleQueueForSequenceIndex()
{
	FScopeLock Lock(&CriticalSection);
	if (!Player.IsValid() || !NextSequenceIndex.IsSet())
	{
		return;
	}
	int32 MinSeqIdx = NextSequenceIndex.GetValue();
	IMediaSamples& Samples = Player->GetSamples();
	Samples.SetMinExpectedNextSequenceIndex(NextSequenceIndex);
}


void FMediaPlayerFacade::UpdateSeekStatus(const FMediaTimeStamp* pCheckTimeStamp)
{
	check(Player.IsValid());

	FScopeLock Lock(&CriticalSection);

	if (HaveVideoPlayback())
	{
		if (IsSeeking())
		{
			// Either peek for the newest available sample or take a given timestamp to check against
			FMediaTimeStamp VideoTimeStamp;
			if (pCheckTimeStamp)
			{
				VideoTimeStamp = *pCheckTimeStamp;
			}
			else
			{
				Player->GetSamples().PeekVideoSampleTime(VideoTimeStamp);
			}

			if (VideoTimeStamp.IsValid() && VideoTimeStamp.GetSequenceIndex() >= NextSequenceIndex.Get(0))
			{
				bool bRunningNonAudioClock = bHaveActiveAudio && !BlockOnRange.IsSet();
				if (bRunningNonAudioClock)
				{
					NextEstVideoTimeAtFrameStart = FMediaTimeStampSample(VideoTimeStamp, FPlatformTime::Seconds());
				}
				FScopeLock LockLT(&TimeValueLock);
				CurrentFrameVideoDisplayTimeStamp = SeekTargetTime;
				SeekTargetTime.Invalidate();
			}
		}
	}
	else
	{
		FScopeLock LockLT(&TimeValueLock);
		if (bHaveActiveAudio)
		{
			if (CurrentFrameAudioTimeStamp >= SeekTargetTime)
			{
				SeekTargetTime.Invalidate();
			}
		}
		else
		{
			// Neither audio nor video are presently active. We just assume we reached the seek target and continue...
			// (we currently have no other source of a current sample timestamp)
			SeekTargetTime.Invalidate();
		}
	}
}

void FMediaPlayerFacade::ExecuteNextSeek()
{
	TimeValueLock.Lock();
	auto nst = NextSeekTime;
	TimeValueLock.Unlock();

	if (nst.IsSet() && !IsSeeking())
	{
		UE_LOGF(LogMediaUtils, Verbose, "PlayerFacade: Internal Seek at %ls.", *nst.GetValue().ToString(TEXT("%h:%m:%s.%t")));

		if (!Seek(nst.GetValue()))
		{
			UE_LOGF(LogMediaUtils, Error, "PlayerFacade: Failed Internal Seek at %ls.", *nst.GetValue().ToString(TEXT("%h:%m:%s.%t")));
		}
	}
}

void FMediaPlayerFacade::MonitorAudioEnablement()
{
	// Update flag reflecting presence of audio in the current stream
	// (doing it just once per gameloop is enough)
	bool bHadActiveAudio = bHaveActiveAudio;
	bHaveActiveAudio = HaveAudioPlayback();
	if (bHadActiveAudio && !bHaveActiveAudio)
	{
		// Reset state for dt-based playback so we grab a new PTS value immediately
		NextEstVideoTimeAtFrameStart.Invalidate();
	}
}


void FMediaPlayerFacade::PreSampleProcessingTimeHandling()
{
	check(Player.IsValid());

	FScopeLock Lock(&CriticalSection);

	PrepareSampleQueueForSequenceIndex();

	UpdateSeekStatus();

	// No seeking?
	if (!IsSeeking())
	{
		// No seek pending & not paused. Can we / Do we need to prime a non-audio clock?
		if (!bHaveActiveAudio && !BlockOnRange.IsSet())
		{
			// Nothing at all?
			if (!NextEstVideoTimeAtFrameStart.IsValid())
			{
				// Try getting a new sample time to start things up...
				FMediaTimeStamp VideoTimeStamp;
				if (Player->GetSamples().PeekVideoSampleTime(VideoTimeStamp))
				{
					NextEstVideoTimeAtFrameStart = FMediaTimeStampSample(VideoTimeStamp, FPlatformTime::Seconds());
				}
			}
			else
			{
				// We have a time. But if we are actively playing forward...
				if (CurrentRate > 0.0f)
				{
					// ...and got some sample waiting for us...
					FMediaTimeStamp VideoTimeStamp;
					if (Player->GetSamples().PeekVideoSampleTime(VideoTimeStamp))
					{
						// ...we need to see if the player's next sample might be so far in the future that we need to re-calibrate our timing
						// (this could happen if the stream has a "gap" in PTS values - e.g. after pausing a live feed from a camera)
						// (^^^ we do not do this on reverse playback as it is unlikely for such streams and might be thinned, hence show gaps under normal conditions)
						if (VideoTimeStamp.GetIndexValue() == NextEstVideoTimeAtFrameStart.TimeStamp.GetIndexValue())
						{
							FTimespan Delta = VideoTimeStamp.Time - NextEstVideoTimeAtFrameStart.TimeStamp.Time;
							if (GetUnpausedRate() < 0.0f)
							{
								Delta = -Delta;
							}

							// Our threshold for re-calibration is twice the length of the last sample we got
							// (or 100ms if we have nothing)
							FTimespan DeltaLimit;
							if (!LastVideoSampleProcessedTimeRange.IsEmpty())
							{
								DeltaLimit = LastVideoSampleProcessedTimeRange.Size<FMediaTimeStamp>().Time * 2;
							}
							else
							{
								DeltaLimit = FTimespan::FromSeconds(0.100);
							}

							if (Delta >= DeltaLimit)
							{
								NextEstVideoTimeAtFrameStart = FMediaTimeStampSample(VideoTimeStamp, FPlatformTime::Seconds());
							}
						}
					}
				}
			}
		}
	}
}


void FMediaPlayerFacade::PostSampleProcessingTimeHandling(FTimespan DeltaTime)
{
	check(Player.IsValid());

	float Rate = CurrentRate;

	// No Audio clock?
	if (!bHaveActiveAudio)
	{
		// No external clock? (blocking)
		if (!BlockOnRange.IsSet())
		{
			// Move video frame start estimate forward
			// (the initial NextEstVideoTimeAtFrameStart will never be valid if no video is present)
			if (NextEstVideoTimeAtFrameStart.IsValid())
			{
				if (Player->GetPlayerFeatureFlag(IMediaPlayer::EFeatureFlag::UseRealtimeWithVideoOnly))
				{
					double NewBaseTime = FPlatformTime::Seconds();
					NextEstVideoTimeAtFrameStart.TimeStamp.Time += FMath::TruncToInt64((NewBaseTime - NextEstVideoTimeAtFrameStart.SampledAtTime) * Rate);
					NextEstVideoTimeAtFrameStart.SampledAtTime = NewBaseTime;
				}
				else
				{
					NextEstVideoTimeAtFrameStart.TimeStamp.Time += DeltaTime * Rate;
				}

				// note: infinite duration (e.g. live playback - or players not yet supporting sequence indices on loops, when looping is enabled)
				// -> no need for special handling as FTimespan::MaxValue() is expected to be returned to signify this, which is quite "infinite" in practical terms
				FTimespan Duration = Player->GetControls().GetDuration();
				const TRange<FTimespan> ActiveRange = GetActivePlaybackRange();
				const FTimespan ActiveRangeStart = ActiveRange.GetLowerBoundValue();
				const FTimespan ActiveRangeEnd = ActiveRange.GetUpperBoundValue();

				if (Player->GetControls().IsLooping())
				{
					if (IsDurationValidAndFinite(Duration))
					{
						const FTimespan ActiveRangeDuration = ActiveRange.GetUpperBoundValue() - ActiveRange.GetLowerBoundValue();
						if (Rate >= 0.0f)
						{
							while(NextEstVideoTimeAtFrameStart.TimeStamp.Time >= ActiveRangeEnd)
							{
								NextEstVideoTimeAtFrameStart.TimeStamp.Time -= ActiveRangeDuration;
								NextEstVideoTimeAtFrameStart.TimeStamp.AdjustLoopIndex(1);
							}
						}
						else
						{
							while(NextEstVideoTimeAtFrameStart.TimeStamp.Time < ActiveRangeStart)
							{
								NextEstVideoTimeAtFrameStart.TimeStamp.Time += ActiveRangeDuration;
								NextEstVideoTimeAtFrameStart.TimeStamp.AdjustLoopIndex(-1);
							}
						}
					}
				}
				else
				{
					if (Rate >= 0.0f)
					{
						if (IsDurationValidAndFinite(Duration))
						{
							if (NextEstVideoTimeAtFrameStart.TimeStamp.Time >= ActiveRangeEnd)
							{
								NextEstVideoTimeAtFrameStart.TimeStamp.Time = ActiveRangeEnd - FTimespan(1);
							}
						}
					}
					else
					{
						if (NextEstVideoTimeAtFrameStart.TimeStamp.Time < ActiveRangeStart)
						{
							NextEstVideoTimeAtFrameStart.TimeStamp.Time = ActiveRangeStart;
						}
					}
				}
			}
		}
	}
}


TRange<FTimespan> FMediaPlayerFacade::GetActivePlaybackRange() const
{
	TRange<FTimespan> Rng(FTimespan::Zero(), FTimespan::Zero());
	if (Player)
	{
		if (SupportsPlaybackTimeRange())
		{
			Rng = GetPlaybackTimeRange(EMediaTimeRangeType::Current);
		}
		else
		{
			FTimespan Duration = Player->GetControls().GetDuration();
			if (Duration <= FTimespan::Zero())
			{
				Duration = FTimespan::MaxValue();
			}
			Rng.SetUpperBound(Duration);
		}
	}
	return Rng;
}

bool FMediaPlayerFacade::GetCurrentPlaybackTimeRange(TRange<FMediaTimeStamp>& TimeRange, float Rate, FTimespan DeltaTime, bool bPurgeSampleRelated) const
{
	/*
	* Note: while a seek operation is still in progress (no sample from target location has been processed) this will
	* return on an empty time range.
	*/
	check(Player->GetPlayerFeatureFlag(IMediaPlayer::EFeatureFlag::UsePlaybackTimingV2));

	TSharedPtr<FMediaAudioSampleSink, ESPMode::ThreadSafe> AudioSink = PrimaryAudioSink.Pin();

	if (bHaveActiveAudio && AudioSink.IsValid())
	{
		//
		// Audio is available...
		//

		FMediaTimeStampSample AudioTime = AudioSink->GetAudioTime();
		if (!AudioTime.IsValid())
		{
			if (!bPurgeSampleRelated)
			{
				// If paused and not seeking, make sure we get one sample nonetheless...
				if (Rate == 0.0f && !IsSeeking())
				{
					// Do this once after open / seek...
					if (LastVideoSampleProcessedTimeRange.IsEmpty())
					{
						// Use the video sample timestamp for simplicity (although we otherwise sync with audio timestamps)
						FMediaTimeStamp TimeStamp;
						if (Player->GetSamples().PeekVideoSampleTime(TimeStamp))
						{
							TimeRange = TRange<FMediaTimeStamp>(TimeStamp, TimeStamp + DeltaTime);
							return !TimeRange.IsEmpty();
						}
					}
				}
			}

			// No timing info available, no time range available, no samples to process
			return false;
		}

		FMediaTimeStamp EstAudioTimeAtFrameStart;

		double Now = FPlatformTime::Seconds();

		if (!bPurgeSampleRelated)
		{
			// Normal estimation relative to current frame start...
			// (on gamethread operation)

			check(IsInGameThread() || IsInSlateThread());

			double AgeOfFrameStart = Now - MediaModule->GetFrameStartTime();
			double AgeOfAudioTime = Now - AudioTime.SampledAtTime;

			if (AgeOfFrameStart >= 0.0 && AgeOfFrameStart <= kMaxTimeSinceFrameStart &&
				AgeOfAudioTime >= 0.0 && AgeOfAudioTime <= kMaxTimeSinceAudioTimeSampling)
			{
				// All realtime timestamps seem in sane ranges - we most likely did not have a lengthy interruption (suspended / debugging step)
				EstAudioTimeAtFrameStart = AudioTime.TimeStamp + FTimespan::FromSeconds((MediaModule->GetFrameStartTime() - AudioTime.SampledAtTime) * Rate);
			}
			else
			{
				// Realtime timestamps seem wonky. Proceed without them (worse estimation quality)
				EstAudioTimeAtFrameStart = AudioTime.TimeStamp;
			}
		}
		else
		{
			// Do not use frame start reference -> we compute relative to "now"
			// (for use off gamethread)
			EstAudioTimeAtFrameStart = AudioTime.TimeStamp + FTimespan::FromSeconds((Now - AudioTime.SampledAtTime) * Rate);
		}

		// Are we paused?
		if (Rate == 0.0f)
		{
			// Yes. We need to fetch a frame for the current display frame - once. Asking over and over until we get one...
			if (LastVideoSampleProcessedTimeRange.IsEmpty())
			{
				// We simply fake the rate to the last non-zero or 1.0 to fetch a frame fitting the time frame representing the whole current frame
				Rate = (LastRate == 0.0f) ? 1.0f : LastRate;
			}
		}

		TimeRange = TRange<FMediaTimeStamp>(EstAudioTimeAtFrameStart, EstAudioTimeAtFrameStart + DeltaTime * FGenericPlatformMath::Abs(Rate));
	}
	else
	{
		//
		// No Audio (no data and/or no sink)
		//
		if (!BlockOnRange.IsSet())
		{
			//
			// Internal clock (DT based)
			//

			// Do we now have a current timestamp estimation?
			if (!NextEstVideoTimeAtFrameStart.IsValid())
			{
				// No timing info available, no time range available, no samples to process
				return false;
			}
			else
			{
				// Yes. Setup current time range & advance time estimation...

				// Are we paused?
				if (Rate == 0.0f)
				{
					// Yes. We need to fetch a frame for the current display frame - once. Asking over and over until we get one...
					if (LastVideoSampleProcessedTimeRange.IsEmpty())
					{
						// We simply fake the rate to the last non-zero or 1.0 to fetch a frame fitting the time frame representing the whole current frame
						Rate = (LastRate == 0.0f) ? 1.0f : LastRate;
					}
				}

				TimeRange = TRange<FMediaTimeStamp>(NextEstVideoTimeAtFrameStart.TimeStamp, NextEstVideoTimeAtFrameStart.TimeStamp + DeltaTime * FGenericPlatformMath::Abs(Rate));
			}
		}
		else
		{
			//
			// External clock delivers time-range
			// (for now we just use the blocking time range as this clock type is solely used in that case)
			//
			TimeRange = GetAdjustedBlockOnRange();
		}
	}

	if (TimeRange.IsEmpty())
	{
		return false;
	}

	const FTimespan Duration = Player->GetControls().GetDuration();
	TRange<FTimespan> ActiveRange = GetActivePlaybackRange();

	// We need a valid duration for the next steps (we may not have one e.g. for live material)
	if (IsDurationValidAndFinite(Duration))
	{
		const FTimespan ActiveRangeDuration = ActiveRange.GetUpperBoundValue() - ActiveRange.GetLowerBoundValue();
		// If we are looping we check to prepare proper ranges should we wrap around either end of the media...
		// (we do not clamp in the non-looping case as the rest of the code should deal with that fine)
		if (Player->GetControls().IsLooping())
		{
			FTimespan WrappedStart = WrappedModulo(TimeRange.GetLowerBoundValue().Time - ActiveRange.GetLowerBoundValue(), ActiveRangeDuration) + ActiveRange.GetLowerBoundValue();
			FTimespan WrappedEnd = WrappedModulo(TimeRange.GetUpperBoundValue().Time - ActiveRange.GetLowerBoundValue(), ActiveRangeDuration) + ActiveRange.GetLowerBoundValue();
			if (WrappedStart > WrappedEnd)
			{
				if (WrappedStart != TimeRange.GetLowerBoundValue().Time)
				{
					TimeRange.SetLowerBoundValue(FMediaTimeStamp(WrappedStart, TimeRange.GetLowerBoundValue().GetSequenceIndex(), TimeRange.GetLowerBoundValue().GetLoopIndex() -1));
				}
				if (WrappedEnd != TimeRange.GetUpperBoundValue().Time)
				{
					TimeRange.SetUpperBoundValue(FMediaTimeStamp(WrappedEnd, TimeRange.GetUpperBoundValue().GetSequenceIndex(), TimeRange.GetUpperBoundValue().GetLoopIndex() + 1));
				}
			}
		}
		else
		{
			TimeRange.SetLowerBoundValue(FMediaTimeStamp(FMath::Clamp(TimeRange.GetLowerBoundValue().Time, ActiveRange.GetLowerBoundValue(), ActiveRange.GetUpperBoundValue()), TimeRange.GetLowerBoundValue().GetSequenceIndex(), TimeRange.GetLowerBoundValue().GetLoopIndex()));
			TimeRange.SetUpperBoundValue(FMediaTimeStamp(FMath::Clamp(TimeRange.GetUpperBoundValue().Time, ActiveRange.GetLowerBoundValue(), ActiveRange.GetUpperBoundValue()), TimeRange.GetUpperBoundValue().GetSequenceIndex(), TimeRange.GetUpperBoundValue().GetLoopIndex()));
		}
	}

	return !TimeRange.IsEmpty();
}


TRange<FMediaTimeStamp> FMediaPlayerFacade::GetAdjustedBlockOnRange() const
{
	TRange<FMediaTimeStamp> TimeRange = BlockOnRange.GetRange();
	return TimeRange;
}


/* FMediaPlayerFacade implementation
*****************************************************************************/

void FMediaPlayerFacade::ProcessAudioSamples(IMediaSamples& Samples, const TRange<FMediaTimeStamp>& TimeRange)
{
	TSharedPtr<IMediaAudioSample, ESPMode::ThreadSafe> Sample;

	check(Player->GetPlayerFeatureFlag(IMediaPlayer::EFeatureFlag::UsePlaybackTimingV2));

	// For V2 we basically expect to get no timerange at all: totally open
	// (we just have it around to be compatible / use older code that expects it)
	check(TimeRange.GetLowerBound().IsOpen() && TimeRange.GetUpperBound().IsOpen());

	//
	// "Modern" 1-Audio-Sink-Only case (aka: we only feed the primary sink)
	//
	if (TSharedPtr< FMediaAudioSampleSink, ESPMode::ThreadSafe> PinnedPrimaryAudioSink = PrimaryAudioSink.Pin())
	{
		while(PinnedPrimaryAudioSink->CanAcceptSamples(1))
		{
			if (!Samples.FetchAudio(TimeRange, Sample))
			{
				break;
			}
			else if (!Sample.IsValid())
			{
				continue;
			}

			{
				FScopeLock Lock(&TimeValueLock);
				LastAudioSampleProcessedTime.TimeStamp = FMediaTimeStamp(Sample->GetTime());
				LastAudioSampleProcessedTime.SampledAtTime = FPlatformTime::Seconds();
			}

			PinnedPrimaryAudioSink->Enqueue(Sample.ToSharedRef());
		}
	}
	else
	{
		// Do we have video playback?
		if (HaveVideoPlayback())
		{
			TRange<FMediaTimeStamp> TempRange;
			// We got video and audio, but no audio sink - throw away anything up to video playback time...
			// (rough estimate, as this is off-gamethread; but better than throwing things out with no throttling at all)
			{
				bool bReverse = (CurrentRate < 0.0f);
				FScopeLock Lock(&TimeValueLock);
				if (!bReverse)
				{
					TempRange.SetUpperBound(CurrentFrameVideoTimeStamp);
				}
				else
				{
					TempRange.SetLowerBound(CurrentFrameVideoTimeStamp);
				}

			}
			while(Samples.FetchAudio(TempRange, Sample))
			{
			}
		}
		else
		{
			// No Video and no primary audio sink: we throw all away (sub-optimal as it will keep audio decoding busy; but this should be an edge case)
			while(Samples.FetchAudio(TimeRange, Sample))
			{
			}
		}
	}
}


bool FMediaPlayerFacade::IsVideoSampleStillGood(const TRange<FMediaTimeStamp>& LastSampleTimeRange, const TRange<FMediaTimeStamp>& TimeRange, bool bReverse) const
{
	// If we have no valid time range or a seek is in progress we assume the current frame can be considered "done" in any case
	if (!TimeRange.IsEmpty() && !IsSeeking() && !LastSampleTimeRange.IsEmpty())
	{
		// This is not the case: check more detailed!

		// This better be true at all times
		check(LastSampleTimeRange.GetLowerBoundValue().GetIndexValue() == LastSampleTimeRange.GetUpperBoundValue().GetIndexValue());

		// - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - -
		// Remap all values so we can assume all of them to be in a single "sequence index range" so the math doesn't get too unruly below.
		// For that the sequence index must not have changed as we otherwise cannot assume any valid frame.
		if (TimeRange.GetLowerBoundValue().GetSequenceIndex() != TimeRange.GetUpperBoundValue().GetSequenceIndex())
		{
			return false;
		}

		FTimespan Duration = Player->GetControls().GetDuration();

		TRange<FMediaTimeStamp> TimeRange0;
		const int32 LoopIndex0 = TimeRange.GetLowerBoundValue().GetLoopIndex();
		const int32 LoopIndex1 = TimeRange.GetUpperBoundValue().GetLoopIndex();
		int32 RefLoopIndex = LoopIndex0;
		// If we encounter a loop we need to check if we can "unroll" it...
		if (LoopIndex0 != LoopIndex1)
		{
			// We only should get here with a looping player that knows its duration
			check(Player->GetControls().IsLooping());
			check(IsDurationValidAndFinite(Duration));

			// Compute how many loops and change the range into one "unrolled" one as indicated by the playback direction...
			int32 LoopIdxDiff = LoopIndex1 - LoopIndex0;
			// Note: this will be positive even with reverse playback as the orientation of the range will no change
			check(LoopIdxDiff > 0);

			double DurationD = Duration.GetTotalSeconds();

			if (!bReverse)
			{
				TimeRange0 = TRange<FMediaTimeStamp>(FMediaTimeStamp(TimeRange.GetLowerBoundValue().Time, 0), FMediaTimeStamp(TimeRange.GetUpperBoundValue().Time + FTimespan::FromSeconds(LoopIdxDiff * DurationD), 0));
			}
			else
			{
				TimeRange0 = TRange<FMediaTimeStamp>(FMediaTimeStamp(TimeRange.GetLowerBoundValue().Time - FTimespan::FromSeconds(LoopIdxDiff * DurationD), 0), FMediaTimeStamp(TimeRange.GetUpperBoundValue().Time, 0));
				RefLoopIndex = LoopIndex1;
			}
		}
		else
		{
			// Simple case, just bring everything down to "zero sequence index" for ease of processing below...
			TimeRange0 = TRange<FMediaTimeStamp>(FMediaTimeStamp(TimeRange.GetLowerBoundValue().Time, 0, 0), FMediaTimeStamp(TimeRange.GetUpperBoundValue().Time, 0, 0));

			// Is looping off?
			if (!Player->GetControls().IsLooping())
			{
				// Yes. We clamp the range to the duration of the video to avoid looking at non-existent "next" frames... (unless we have no duration)
				if (IsDurationValidAndFinite(Duration))
				{
					const TRange<FTimespan> ActiveRange = GetActivePlaybackRange();
					TimeRange0 = TRange<FMediaTimeStamp>::Intersection(TimeRange0, TRange<FMediaTimeStamp>(FMediaTimeStamp(ActiveRange.GetLowerBoundValue(), 0), FMediaTimeStamp(ActiveRange.GetUpperBoundValue(), 0)));
				}
			}
		}

		// Map the last sample's time range to the same "sequence index" range as the time range
		// (note: for Live streams that do not have any set duration all this will not change the timerange - just as needed)
		const int32 LastSampleLoopDiff = LastSampleTimeRange.GetLowerBoundValue().GetLoopIndex() - RefLoopIndex;
		FTimespan TimeOffset = IsDurationValidAndFinite(Duration) ? Duration * LastSampleLoopDiff : FTimespan::Zero();
		TRange<FMediaTimeStamp> LastSampleTimeRange0(FMediaTimeStamp(LastSampleTimeRange.GetLowerBoundValue().Time + TimeOffset, 0, 0), FMediaTimeStamp(LastSampleTimeRange.GetUpperBoundValue().Time + TimeOffset, 0, 0));

		// - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - -
		// Now we can begin the checks with all time ranges mapped back to "zero sequence index"

		// Is the sample time range ahead of the given time range?
		// (did the range move in an unexpected way?)
		if (!bReverse ? TimeRange0.GetUpperBoundValue() <= LastSampleTimeRange0.GetLowerBoundValue()
					  : TimeRange0.GetLowerBoundValue() >= LastSampleTimeRange0.GetUpperBoundValue())
		{
			// We simply let the last sample stay around...
			return true;
		}

		// Is the sample time range at all still valid?
		if (LastSampleTimeRange0.Overlaps(TimeRange0))
		{
			// Yes. Assuming we could get more samples (of the same type) from the player, would the next one be "better"?
			// (we assume samples of equal length)

			// Compute the "theoretical" next sample range...
			TRange<FMediaTimeStamp> NextSampleTimeRange = !bReverse ? TRange<FMediaTimeStamp>(LastSampleTimeRange0.GetUpperBoundValue(), LastSampleTimeRange0.GetUpperBoundValue() + LastSampleTimeRange0.Size<FMediaTimeStamp>().Time)
																	: TRange<FMediaTimeStamp>(LastSampleTimeRange0.GetLowerBoundValue() - LastSampleTimeRange0.Size<FMediaTimeStamp>().Time, LastSampleTimeRange0.GetLowerBoundValue());

			// Note: Loops (or the end of the time line in non-looping setups)
			//
			// - We could check for them and generate proper changes to the sequence index
			// - Doing this would leave us with quite complex setups to compute the coverage
			// - We opt for a cleaner, simpler approach: as we are NOT interested in proper PTS values here, we can safely work with an "infinite" time line when computing any overlaps, coverage and such
			//   (note: we DO need to restrict the range to the actual media duration if not looping - the code above does this)
			//
			// --> we simply keep what we compute above!
			//

			// Compute which one is larger inside the current range...
			int64 LastSampleCoverage = TRange<FMediaTimeStamp>::Intersection(TimeRange0, LastSampleTimeRange0).Size<FMediaTimeStamp>().Time.GetTicks();
			int64 NextSampleCoverage = TRange<FMediaTimeStamp>::Intersection(TimeRange0, NextSampleTimeRange).Size<FMediaTimeStamp>().Time.GetTicks();

			// A new one is only desirable if it's BETTER than the current one
			if (LastSampleCoverage >= NextSampleCoverage)
			{
				// Last one we returned is still good. No new one needed...
				return true;
			}
		}
	}
	return false;
}


bool FMediaPlayerFacade::ProcessVideoSamples(IMediaSamples& Samples, const TRange<FMediaTimeStamp>& TimeRange)
{
	if (!Player.IsValid())
	{
		// Nothing to do, but in a sense: "successful"...
		return true;
	}

	// This is not to be used with V1 timing
	check(Player->GetPlayerFeatureFlag(IMediaPlayer::EFeatureFlag::UsePlaybackTimingV2));
	// We expect a fully closed range or we assume: nothing to do...
	check(TimeRange.GetLowerBound().IsClosed() && TimeRange.GetUpperBound().IsClosed());

	TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe> Sample;

	if (!Player->GetPlayerFeatureFlag(IMediaPlayer::EFeatureFlag::AlwaysPullNewestVideoFrame))
	{
		//
		// Normal playback with timing control provided by MediaFramework
		//
		const bool bReverse = (GetUnpausedRate() < 0.0f);

		if (IsVideoSampleStillGood(LastVideoSampleProcessedTimeRange, TimeRange, bReverse))
		{
			// We got all the samples we need. Processing was successful...
			return true;
		}

		switch(Samples.FetchBestVideoSampleForTimeRange(TimeRange, Sample, bReverse, BlockOnRange.IsSet()))
		{
			case IMediaSamples::EFetchBestSampleResult::Ok:
			{
				break;
			}

			case IMediaSamples::EFetchBestSampleResult::NoSample:
			{
				break;
			}

			case IMediaSamples::EFetchBestSampleResult::PurgedToEmpty:
			{
				// When there is no audio to sync to then we are extrapolating the next expected video timestamp
				// from the last plus the elapsed deltatime, which may overshoot the next decoder output.
				// In this case we resynchronize the timestamp to the next available video frame.
				if (!HaveAudioPlayback())
				{
					NextEstVideoTimeAtFrameStart.Invalidate();
				}
				break;
			}

			case IMediaSamples::EFetchBestSampleResult::NotSupported:
			{
				//
				// Fallback for players supporting V2 timing, but do not supply FetchBestVideoSampleForTimeRange() due to some
				// custom implementation of IMediaSamples (here to ease adoption of the new timing code - eventually should go away)
				//

				// Find newest sample that satisfies the time range
				// (the FetchXYZ() code does not work well with a lower range limit at all - we ask for a "up to" type range instead
				//  and limit the other side of the range in code here to not change the older logic & possibly cause trouble in old code)
				TRange<FMediaTimeStamp> TempRange = bReverse ? TRange<FMediaTimeStamp>::AtLeast(TimeRange.GetUpperBoundValue()) : TRange<FMediaTimeStamp>::AtMost(TimeRange.GetUpperBoundValue());
				while(Samples.FetchVideo(TempRange, Sample))
				{ }
				if (Sample.IsValid() &&
					((!bReverse && ((Sample->GetTime() + Sample->GetDuration()) > TimeRange.GetLowerBoundValue())) ||
						(bReverse && ((Sample->GetTime() - Sample->GetDuration()) < TimeRange.GetLowerBoundValue()))))
				{
					// Sample is good - nothing more to do here
				}
				else
				{
					Sample.Reset();
				}
				break;
			}
		}
	}
	else
	{
		//
		// Use newest video frame available at all times (no Mediaframework timing control)
		//
		TRange<FMediaTimeStamp> TempRange; // fully open range
		while(Samples.FetchVideo(TempRange, Sample))
		{ }
	}

	// Any sample?
	if (Sample.IsValid())
	{
		// Yes, deliver it and update state...

		FMediaTimeStamp SampleTime = Sample->GetTime();
		TRange<FMediaTimeStamp> SampleTimeRange(SampleTime, SampleTime + Sample->GetDuration());

#if !NO_LOGGING
		if (VideoSampleSinks.IsEmpty())
		{
			UE_LOGF(LogMediaUtils, Verbose, "PlayerFacade: No Video Sample Sinks available, Dropping Sample %ls Sequence %d Loop %d",
				*SampleTime.GetTime().ToString(TEXT("%h:%m:%s.%t")), SampleTime.GetSequenceIndex(), SampleTime.GetLoopIndex());
		}
		else
		{
			UE_LOGF(LogMediaUtils, VeryVerbose, "PlayerFacade: Enqueuing Sample %ls Sequence %d Loop %d",
				*SampleTime.GetTime().ToString(TEXT("%h:%m:%s.%t")), SampleTime.GetSequenceIndex(), SampleTime.GetLoopIndex());
		}
#endif

		// Enqueue the sample to render
		// (we use a queue to stay compatible with existing structure and older sinks - new sinks will read this single entry right away on the gamethread
		//  and pass it along to rendering outside the queue)
		bool bOk = VideoSampleSinks.Enqueue(Sample.ToSharedRef());
		check(bOk);

		{
			FScopeLock Lock(&TimeValueLock);
			CurrentFrameVideoDisplayTimeStamp = CurrentFrameVideoTimeStamp = SampleTimeRange.GetLowerBoundValue();
			LastVideoSampleProcessedTimeRange = SampleTimeRange;
			MostRecentlyDeliveredVideoFrameTimecode = Sample->GetTimecode();
		}

		UpdateSeekStatus(&CurrentFrameVideoTimeStamp);
		return true;
	}
	return false;
}


void FMediaPlayerFacade::ProcessCaptionSamples(IMediaSamples& Samples, const TRange<FMediaTimeStamp>& TimeRange)
{
	check(Player->GetPlayerFeatureFlag(IMediaPlayer::EFeatureFlag::UsePlaybackTimingV2));

	TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe> Sample;

	// Seek in progress?
	if (IsSeeking())
	{
		// Yes. Fetch (and discard) all samples up to the seek target time...
		// (we only throw out samples from prior sequence indices to make sure we do not swallow any audio from overlapping samples)
		FScopeLock Lock(&TimeValueLock);
		auto DiscardRange = TRange<FMediaTimeStamp>(FMediaTimeStamp(0), FMediaTimeStamp((CurrentRate >= 0.0f) ? FTimespan::Zero() : FTimespan::MaxValue(), SeekTargetTime.GetSequenceIndex(), SeekTargetTime.GetLoopIndex()));
		Samples.DiscardCaptionSamples(DiscardRange, GetUnpausedRate() < 0.0f);
	}
	else
	{
		while(Samples.FetchCaption(TimeRange, Sample))
		{
			if (Sample.IsValid() && !CaptionSampleSinks.Enqueue(Sample.ToSharedRef()))
			{
#if MEDIAPLAYERFACADE_TRACE_SINKOVERFLOWS
				UE_LOGF(LogMediaUtils, VeryVerbose, "PlayerFacade: Caption sample sink overflow");
#endif
			}
		}
	}
}


void FMediaPlayerFacade::ProcessSubtitleSamples(IMediaSamples& Samples, const TRange<FMediaTimeStamp>& TimeRange)
{
	check(Player->GetPlayerFeatureFlag(IMediaPlayer::EFeatureFlag::UsePlaybackTimingV2));

	TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe> Sample;

	// Seek in progress?
	if (IsSeeking())
	{
		// Yes. Fetch (and discard) all samples up to the seek target time...
		// (we only throw out samples from prior sequence indices to make sure we do not swallow any audio from overlapping samples)
		FScopeLock Lock(&TimeValueLock);
		auto DiscardRange = TRange<FMediaTimeStamp>(FMediaTimeStamp(0), FMediaTimeStamp((CurrentRate >= 0.0f) ? FTimespan::Zero() : FTimespan::MaxValue(), SeekTargetTime.GetSequenceIndex(), SeekTargetTime.GetLoopIndex()));
		Samples.DiscardSubtitleSamples(DiscardRange, GetUnpausedRate() < 0.0f);
	}
	else
	{
		const bool bReverse = CurrentRate < 0.0f;
		TRange<FMediaTimeStamp> TempRange = bReverse ? TRange<FMediaTimeStamp>::AtLeast(TimeRange.GetUpperBoundValue()) : TRange<FMediaTimeStamp>::AtMost(TimeRange.GetUpperBoundValue());
		while(Samples.FetchSubtitle(TempRange, Sample))
		{ }
		if (Sample.IsValid() &&
			((!bReverse && ((Sample->GetTime() + Sample->GetDuration()) > TimeRange.GetLowerBoundValue())) ||
				(bReverse && ((Sample->GetTime() - Sample->GetDuration()) < TimeRange.GetLowerBoundValue()))))
		{
			//UE_LOGF(LogMediaUtils, Display, "Subtitle @%.3f: %ls", Sample->GetTime().Time.GetTotalSeconds(), *Sample->GetText().ToString());
			if (Sample.IsValid() && !SubtitleSampleSinks.Enqueue(Sample.ToSharedRef()))
			{
#if MEDIAPLAYERFACADE_TRACE_SINKOVERFLOWS
				UE_LOGF(LogMediaUtils, VeryVerbose, "PlayerFacade: Subtitle sample sink overflow");
#endif
			}
		}
	}
}


void FMediaPlayerFacade::ProcessMetadataSamples(IMediaSamples& Samples, const TRange<FMediaTimeStamp>& TimeRange)
{
	check(Player->GetPlayerFeatureFlag(IMediaPlayer::EFeatureFlag::UsePlaybackTimingV2));

	TSharedPtr<IMediaBinarySample, ESPMode::ThreadSafe> Sample;

	// Seek in progress?
	if (IsSeeking())
	{
		// Yes. Fetch (and discard) all samples up to the seek target time...
		// (we only throw out samples from prior sequence indices to make sure we do not swallow any audio from overlapping samples)
		FScopeLock Lock(&TimeValueLock);
		auto DiscardRange = TRange<FMediaTimeStamp>(FMediaTimeStamp(0), FMediaTimeStamp((CurrentRate >= 0.0f) ? FTimespan::Zero() : FTimespan::MaxValue(), SeekTargetTime.GetSequenceIndex(), SeekTargetTime.GetLoopIndex()));
		Samples.DiscardMetadataSamples(DiscardRange, GetUnpausedRate() < 0.0f);
	}
	else
	{
		while(Samples.FetchMetadata(TimeRange, Sample))
		{
			if (Sample.IsValid() && !MetadataSampleSinks.Enqueue(Sample.ToSharedRef()))
			{
#if MEDIAPLAYERFACADE_TRACE_SINKOVERFLOWS
				UE_LOGF(LogMediaUtils, VeryVerbose, "PlayerFacade: Metadata sample sink overflow");
#endif
			}
		}
	}
}


void FMediaPlayerFacade::ProcessAudioSamplesV1(IMediaSamples& Samples, TRange<FTimespan> TimeRange)
{
	TSharedPtr<IMediaAudioSample, ESPMode::ThreadSafe> Sample;

	while(Samples.FetchAudio(TimeRange, Sample))
	{
		if (!Sample.IsValid())
		{
			continue;
		}

		if (!AudioSampleSinks.Enqueue(Sample.ToSharedRef()))
		{
#if MEDIAPLAYERFACADE_TRACE_SINKOVERFLOWS
			UE_LOGF(LogMediaUtils, VeryVerbose, "PlayerFacade: Audio sample sink overflow");
#endif
		}
		else
		{
			FScopeLock Lock(&TimeValueLock);
			LastAudioSampleProcessedTime.TimeStamp = Sample->GetTime();
			LastAudioSampleProcessedTime.SampledAtTime = FPlatformTime::Seconds();
		}
	}
}


void FMediaPlayerFacade::ProcessVideoSamplesV1(IMediaSamples& Samples, TRange<FTimespan> TimeRange)
{
	// This is not to be used with V2 timing
	check(!Player->GetPlayerFeatureFlag(IMediaPlayer::EFeatureFlag::UsePlaybackTimingV2));

	TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe> Sample;

	while(Samples.FetchVideo(TimeRange, Sample))
	{
		if (!Sample.IsValid())
		{
			continue;
		}

		{
			FScopeLock Lock(&TimeValueLock);
			CurrentFrameVideoDisplayTimeStamp = CurrentFrameVideoTimeStamp = Sample->GetTime();
		}

		UE_LOGF(LogMediaUtils, VeryVerbose, "PlayerFacade: Fetched video sample %ls", *Sample->GetTime().Time.ToString(TEXT("%h:%m:%s.%t")));

		if (VideoSampleSinks.Enqueue(Sample.ToSharedRef()))
		{
			if (CurrentRate >= 0.0f)
			{
				NextVideoSampleTime = Sample->GetTime().Time + Sample->GetDuration();
				UE_LOGF(LogMediaUtils, VeryVerbose, "PlayerFacade: Next video sample time %ls", *NextVideoSampleTime.ToString(TEXT("%h:%m:%s.%t")));
			}
		}
		else
		{
#if MEDIAPLAYERFACADE_TRACE_SINKOVERFLOWS
			UE_LOGF(LogMediaUtils, VeryVerbose, "PlayerFacade: Video sample sink overflow");
#endif
		}
	}
}

void FMediaPlayerFacade::ProcessCaptionSamplesV1(IMediaSamples& Samples, TRange<FTimespan> TimeRange)
{
	TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe> Sample;

	while(Samples.FetchCaption(TimeRange, Sample))
	{
		if (Sample.IsValid() && !CaptionSampleSinks.Enqueue(Sample.ToSharedRef()))
		{
#if MEDIAPLAYERFACADE_TRACE_SINKOVERFLOWS
			UE_LOGF(LogMediaUtils, VeryVerbose, "PlayerFacade: Caption sample sink overflow");
#endif
		}
	}
}


void FMediaPlayerFacade::ProcessSubtitleSamplesV1(IMediaSamples& Samples, TRange<FTimespan> TimeRange)
{
	TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe> Sample;

	while(Samples.FetchSubtitle(TimeRange, Sample))
	{
		if (Sample.IsValid() && !SubtitleSampleSinks.Enqueue(Sample.ToSharedRef()))
		{
			FString Caption = Sample->GetText().ToString();
			//UE_LOGF(LogMediaUtils, Log, "New caption @%.3f: %ls", Sample->GetTime().Time.GetTotalSeconds(), *Caption);

#if MEDIAPLAYERFACADE_TRACE_SINKOVERFLOWS
			UE_LOGF(LogMediaUtils, VeryVerbose, "PlayerFacade: Subtitle sample sink overflow");
#endif
		}
	}
}


void FMediaPlayerFacade::ProcessMetadataSamplesV1(IMediaSamples& Samples, TRange<FTimespan> TimeRange)
{
	TSharedPtr<IMediaBinarySample, ESPMode::ThreadSafe> Sample;

	while(Samples.FetchMetadata(TimeRange, Sample))
	{
		if (Sample.IsValid() && !MetadataSampleSinks.Enqueue(Sample.ToSharedRef()))
		{
#if MEDIAPLAYERFACADE_TRACE_SINKOVERFLOWS
			UE_LOGF(LogMediaUtils, VeryVerbose, "PlayerFacade: Metadata sample sink overflow");
#endif
		}
	}
}


/* IMediaEventSink interface
*****************************************************************************/

void FMediaPlayerFacade::ReceiveMediaEvent(EMediaEvent Event)
{
	if (Event >= EMediaEvent::Internal_Start)
	{
		switch (Event)
		{
			case EMediaEvent::Internal_PurgeVideoSamplesHint:
			{
				/*
					Sent by some media players to ask to purge older samples from the video output queue.
					This is done to ensure that, even if the game thread is stalled and the facade is not
					being ticked regularly where it would perform this task by passing frames from the
					queue to the sink, frames that have passed the point where they should have been
					sent to the sink will not clog the queue.
					The player cannot perform this task on its own because it does not know the current
					precise playback position.

					Here we need to handle only everything not audio because audio is pulled by the
					audio thread and not the gamethread, so it can never stall.
				*/
				TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CurrentPlayer = Player;

				if (!CurrentPlayer.IsValid())
				{
					return;
				}

				// We only support this for V2 timing players
				check(CurrentPlayer->GetPlayerFeatureFlag(IMediaPlayer::EFeatureFlag::UsePlaybackTimingV2));

				// Only do this if we do not block on time ranges
				if (BlockOnRange.IsSet())
				{
					// We do not purge as we do not need max perf, but max reliability to actually get certain frames
					return;
				}

				const float Rate = CurrentRate;
				if (Rate == 0.0f)
				{
					return;
				}

				// Get current playback time
				// (Note: the delta time is entirely synthetic - we do not pass zero to avoid an empty range, but we do not look far into the future either
				//        -> after all: we are mainly focused on purging samples up to the current time
				//  Remarks:
				//   - this version does not take any estimations from any frame start into account as this is entirely async to the main thread
				//   - video streams with no audio content will be played using the UE DeltaTime -> so if that stops, the progress of the video stops!
				//     -> hence we will not see (other then one initial purge) any purging of samples here!
				// )
				TRange<FMediaTimeStamp> TimeRange;
				if (!GetCurrentPlaybackTimeRange(TimeRange, Rate, FTimespan::FromMilliseconds(kOutdatedSamplePurgeRange), true))
				{
					return;
				}

				bool bReverse = (Rate < 0.0f);
				const float RateFactor = (Rate != 0.0f) ? (1.0f / Rate) : 1.0f;

				// Don't purge frames if the queue is small (to avoid purging if players deliver frames late persistently)
				uint32 NumPurged = 0;
				if (CurrentPlayer->GetSamples().NumVideoSamples() >= kMinFramesInVideoQueueToPurge)
				{
					NumPurged = CurrentPlayer->GetSamples().PurgeOutdatedVideoSamples(TimeRange.GetLowerBoundValue(), bReverse, FTimespan::FromSeconds(kOutdatedVideoSamplesTolerance * RateFactor));
				}
				SET_DWORD_STAT(STAT_MediaUtils_FacadeNumPurgedVideoSamples, NumPurged);
				INC_DWORD_STAT_BY(STAT_MediaUtils_FacadeTotalPurgedVideoSamples, NumPurged);

				// Take the opportunity to also purge any samples related to video samples directly (and evaluated on the game thread)

				// Captions...
				NumPurged = 0;
				if (CurrentPlayer->GetSamples().NumCaptionSamples() >= kMinFramesInCaptionQueueToPurge)
				{
					NumPurged = CurrentPlayer->GetSamples().PurgeOutdatedCaptionSamples(TimeRange.GetLowerBoundValue(), bReverse, FTimespan::FromSeconds(kOutdatedSubtitleSamplesTolerance * RateFactor));
				}
				SET_DWORD_STAT(STAT_MediaUtils_FacadeNumPurgedSubtitleSamples, NumPurged);
				INC_DWORD_STAT_BY(STAT_MediaUtils_FacadeTotalPurgedSubtitleSamples, NumPurged);

				// Subtitles...
				NumPurged = 0;
				if (CurrentPlayer->GetSamples().NumSubtitleSamples() >= kMinFramesInSubtitleQueueToPurge)
				{
					NumPurged = CurrentPlayer->GetSamples().PurgeOutdatedSubtitleSamples(TimeRange.GetLowerBoundValue(), bReverse, FTimespan::FromSeconds(kOutdatedSubtitleSamplesTolerance * RateFactor));
				}
				SET_DWORD_STAT(STAT_MediaUtils_FacadeNumPurgedCaptionSamples, NumPurged);
				INC_DWORD_STAT_BY(STAT_MediaUtils_FacadeTotalPurgedCaptionSamples, NumPurged);
				break;
			}

			case EMediaEvent::Internal_VideoSamplesAvailable:
			{
				UE_LOGF(LogMediaUtils, VeryVerbose, "PlayerFacade: Video samples ARE available");
				VideoSampleAvailability = 1;
				break;
			}
			case EMediaEvent::Internal_VideoSamplesUnavailable:
			{
				UE_LOGF(LogMediaUtils, VeryVerbose, "PlayerFacade: Video samples are NOT available");
				VideoSampleAvailability = 0;
				break;
			}
			case EMediaEvent::Internal_AudioSamplesAvailable:
			{
				UE_LOGF(LogMediaUtils, VeryVerbose, "PlayerFacade: Audio samples ARE available");
				AudioSampleAvailability = 1;
				break;
			}
			case EMediaEvent::Internal_AudioSamplesUnavailable:
			{
				UE_LOGF(LogMediaUtils, VeryVerbose, "PlayerFacade: Audio samples are NOT available");
				AudioSampleAvailability = 0;
				break;
			}

			case EMediaEvent::Internal_RequestVideoSampleRefresh:
			{
				// Clear the "last sample still good" cache so the next TickInput re-fetches even if player is paused.
				UE_LOGF(LogMediaUtils, VeryVerbose, "PlayerFacade [%ls]: Request video sample refresh", *PlayerGuid.ToString());
				FScopeLock Lock(&TimeValueLock);
				LastVideoSampleProcessedTimeRange = TRange<FMediaTimeStamp>::Empty();
				break;
			}

			default:
			{
				UE_LOGF(LogMediaUtils, VeryVerbose, "PlayerFacade: Received media event %ls", *MediaUtils::EventToString(Event));
				break;
			}
		}
	}
	else
	{
		UE_LOGF(LogMediaUtils, VeryVerbose, "PlayerFacade: Received media event %ls", *MediaUtils::EventToString(Event));
		QueuedEvents.Enqueue(Event);
	}
}

void FMediaPlayerFacade::ReInit()
{
	// We leave the registered sinks and delegates alone
	{
		FScopeLock lock(&CriticalSection);
		BlockOnRange.Reset();
		BlockOnRangeDisabled = false;
		CurrentUrl.Empty();
		LastRate = 0.0f;
		CurrentRate = 0.0f;
		bHaveActiveAudio = false;
		VideoSampleAvailability = -1;
		AudioSampleAvailability = -1;
		NextVideoSampleTime = FTimespan::Zero();
	}

	{
		FScopeLock lock(&TimeValueLock);
		LastAudioRenderedSampleTime.Invalidate();
		LastAudioSampleProcessedTime.Invalidate();
		LastVideoSampleProcessedTimeRange = TRange<FMediaTimeStamp>::Empty();
		CurrentFrameAudioTimeStamp.Invalidate();
		CurrentFrameVideoTimeStamp.Invalidate();
		CurrentFrameVideoDisplayTimeStamp.Invalidate();
		NextEstVideoTimeAtFrameStart.Invalidate();
		MostRecentlyDeliveredVideoFrameTimecode.Reset();
		SeekTargetTime.Invalidate();
		NextSeekTime.Reset();
		NextSequenceIndex.Reset();
	}
}

