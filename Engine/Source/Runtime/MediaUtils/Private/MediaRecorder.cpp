// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaRecorder.h"

#include "Async/Async.h"
#include "Containers/UnrealString.h"
#include "HAL/FileManager.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "ImagePixelData.h"
#include "ImageWriteQueue.h"
#include "IMediaClock.h"
#include "IMediaClockSink.h"
#include "IMediaModule.h"
#include "Math/UnrealMathUtility.h"
#include "MediaUtilsPrivate.h"
#include "Modules/ModuleManager.h"
#include "Serialization/Archive.h"

#include "MediaPlayerFacade.h"


/* Local helpers
 *****************************************************************************/

/**
 * Media clock sink for media textures.
 */
class FMediaRecorderClockSink
	: public IMediaClockSink
{
public:

	FMediaRecorderClockSink(FMediaRecorder& InOwner)
		: Owner(InOwner)
	{ }

	virtual ~FMediaRecorderClockSink() { }

public:

	virtual void TickOutput(FTimespan DeltaTime, FTimespan Timecode) override
	{
		Owner.TickRecording();
	}

private:

	FMediaRecorder& Owner;
};


/**
 * ImagePixelData for TextureSample.
 * Can only be used when Stride == Dim.X*"Number of channels"
 */
struct FMediaImagePixelData : FImagePixelData
{
	FMediaImagePixelData(TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& InSample
		, const FIntPoint& InSize
		, EImagePixelType InPixelType
		, ERGBFormat InPixelLayout
		, uint8 InBitDepth
		, uint8 InNumChannels)
		: FImagePixelData(InSize, InPixelType, InPixelLayout, InBitDepth, InNumChannels, nullptr)
		, Sample(InSample)
	{
	}

	TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe> Sample;

	virtual TUniquePtr<FImagePixelData> Move() override
	{
		return MakeUnique<FMediaImagePixelData>(MoveTemp(*this));
	}

	virtual TUniquePtr<FImagePixelData> Copy() const override
	{
		return MakeUnique<FMediaImagePixelData>(*this);
	}

	virtual void RetrieveData(const void*& OutDataPtr, int64& OutSizeBytes) const override
	{
		OutDataPtr = Sample->GetBuffer();
		OutSizeBytes = int64(Sample->GetStride()) * int64(Sample->GetDim().Y);
	}
};


/* MediaRecorderHelpers namespace
*****************************************************************************/

namespace MediaRecorderHelpers
{
	template<class TColorType>
	TUniquePtr<TImagePixelData<TColorType>> CreatePixelData(TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe> InSample, const FIntPoint InSize, int32 InNumChannels)
	{
		const int64 NumberOfTexel = InSize.Y * InSize.X;
		TUniquePtr<TImagePixelData<TColorType>> PixelData = MakeUnique<TImagePixelData<TColorType>>(InSize);
		PixelData->Pixels.Reset(NumberOfTexel);

		const void* Buffer = InSample->GetBuffer();
		const uint64 Stride = InSample->GetStride();
		if (NumberOfTexel == (Stride *InSize.Y) / InNumChannels)
		{
			PixelData->Pixels.Append(reinterpret_cast<const TColorType*>(Buffer), NumberOfTexel);
		}
		else
		{
			for (int64 IndexY = 0; IndexY < InSize.Y; ++IndexY)
			{
				PixelData->Pixels.Append(reinterpret_cast<const TColorType*>(reinterpret_cast<const uint8*>(Buffer) + (Stride*IndexY)), InSize.X);
			}
		}

		return MoveTemp(PixelData);
	};
}

/* FMediaRecorder implementation
 *****************************************************************************/

FMediaRecorder::FMediaRecorder()
	: bRecording(false)
	, bUnsupportedWarningShowed(false)
	, FrameCount(0)
	, NumerationStyle(EMediaRecorderNumerationStyle::AppendSampleTime)
	, bSetAlpha(false)
	, CompressionQuality(0)
	, ImageWriteQueue(nullptr)
{ }


void FMediaRecorder::StartRecording(const FMediaRecorderData& InRecoderData)
{
	if (bRecording)
	{
		StopRecording();
	}

	IMediaModule* MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media");

	if (MediaModule == nullptr)
	{
		return;
	}

	BaseFilename = InRecoderData.BaseFilename;
	NumerationStyle = InRecoderData.NumerationStyle;
	TargetImageFormat = InRecoderData.TargetImageFormat;
	bSetAlpha = InRecoderData.bResetAlpha;
	CompressionQuality = InRecoderData.CompressionQuality;

	SampleQueue = MakeShared<FMediaTextureSampleQueue, ESPMode::ThreadSafe>();
	InRecoderData.PlayerFacade->AddVideoSampleSink(SampleQueue.ToSharedRef());

	ClockSink = MakeShared<FMediaRecorderClockSink, ESPMode::ThreadSafe>(*this);
	MediaModule->GetClock().AddSink(ClockSink.ToSharedRef());

	ImageWriteQueue = &FModuleManager::LoadModuleChecked<IImageWriteQueueModule>("ImageWriteQueue").GetWriteQueue();

	bRecording = true;
}


void FMediaRecorder::StopRecording()
{
	if (bRecording && ImageWriteQueue)
	{
		CompletedFence = ImageWriteQueue->CreateFence();
	}
	bRecording = false;

	ImageWriteQueue = nullptr;
	ClockSink.Reset();
	SampleQueue.Reset();
}


bool FMediaRecorder::WaitPendingTasks(const FTimespan& InDuration)
{
	bool bResult = true;
	if (CompletedFence.IsValid())
	{
		bResult = CompletedFence.WaitFor(InDuration);
	}
	CompletedFence = TFuture<void>();
	return bResult;
}


void FMediaRecorder::TickRecording()
{
	if (!bRecording)
	{
		return; // not recording
	}

	check(SampleQueue.IsValid());

	if (ImageWriteQueue == nullptr)
	{
		while (SampleQueue->Pop());
		StopRecording();
	}
	else
	{
		TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe> Sample;

		while (SampleQueue->Dequeue(Sample) && Sample.IsValid())
		{
			const void* SampleBuffer = Sample->GetBuffer();

			if (SampleBuffer == nullptr)
			{
				continue; // only raw samples supported right now
			}

			if (Sample->GetDim().GetMin() == 0)
			{
				continue; // nothing to save
			}

			if (Sample->GetFormat() != EMediaTextureSampleFormat::CharBGRA && Sample->GetFormat() != EMediaTextureSampleFormat::FloatRGBA)
			{
				if (!bUnsupportedWarningShowed)
				{
					UE_LOGF(LogMediaUtils, Warning, "Media Source type cannot be recorded by Sequence Recorder. Texture Sample Format '%ls' is not supported by Media Recorder.", MediaTextureSampleFormat::EnumToString(Sample->GetFormat()));
					bUnsupportedWarningShowed = true;
				}
				continue;
			}

			TUniquePtr<FImageWriteTask> ImageTask = MakeUnique<FImageWriteTask>();

			// Set PixelData
			{
				const FIntPoint Size = Sample->GetDim();
				EImagePixelType PixelType = EImagePixelType::Color;
				ERGBFormat PixelLayout = ERGBFormat::BGRA;
				uint8 BitDepth = 8;
				uint8 NumChannels = 4;

				if (Sample->GetFormat() == EMediaTextureSampleFormat::FloatRGBA)
				{
					PixelType = EImagePixelType::Float16;
					PixelLayout = ERGBFormat::RGBAF;
					BitDepth = 16;
				}

				// Should we move the color buffer into a raw image data container.
				bool bUseFMediaImagePixelData = bSetAlpha || (Sample->GetStride() != Size.X * NumChannels * (BitDepth/8));

				if (bUseFMediaImagePixelData)
				{
					const int32 NumberOfTexel = Size.Y * Size.X;
					if (Sample->GetFormat() == EMediaTextureSampleFormat::FloatRGBA)
					{
						ImageTask->PixelData = MediaRecorderHelpers::CreatePixelData<FFloat16Color>(Sample, Size, NumChannels);
					}
					else
					{
						check(Sample->GetFormat() == EMediaTextureSampleFormat::CharBGRA);
						ImageTask->PixelData = MediaRecorderHelpers::CreatePixelData<FColor>(Sample, Size, NumChannels);
					}
				}
				else
				{
					// Use the MediaSample to save memory
					ImageTask->PixelData = MakeUnique<FMediaImagePixelData>(Sample, Size, PixelType, PixelLayout, BitDepth, NumChannels);
				}

				if (bSetAlpha)
				{
					check(bUseFMediaImagePixelData);

					ImageTask->AddPreProcessorToSetAlphaOpaque();
				}
			}
			
			if (Sample->GetFormat() == EMediaTextureSampleFormat::CharBGRA)
			{
				// IsOutputSrgb almost always true
				ImageTask->PixelData->SetSRGB( Sample->IsOutputSrgb() );
			}

			ImageTask->Format = TargetImageFormat;
			ImageTask->CompressionQuality = CompressionQuality;
			ImageTask->bOverwriteFile = true;
			if (NumerationStyle == EMediaRecorderNumerationStyle::AppendFrameNumber)
			{
				ImageTask->Filename = FString::Printf(TEXT("%s_%08d"), *BaseFilename, FrameCount);
			}
			else
			{
				ImageTask->Filename = FString::Printf(TEXT("%s_%.16" INT64_FMT), *BaseFilename, Sample->GetTime().Time.GetTicks());
			}

			ImageWriteQueue->Enqueue(MoveTemp(ImageTask), false);
			++FrameCount;
		}
	}
}
