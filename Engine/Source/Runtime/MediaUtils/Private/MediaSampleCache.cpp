// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaSampleCache.h"
#include "MediaUtilsPrivate.h"

#include "IMediaAudioSample.h"
#include "IMediaBinarySample.h"
#include "IMediaControls.h"
#include "IMediaOverlaySample.h"
#include "IMediaPlayer.h"
#include "IMediaSamples.h"
#include "IMediaTextureSample.h"
#include "Math/Range.h"
#include "Misc/ScopeLock.h"


/* FMediaSampleCache structors
*****************************************************************************/

FMediaSampleCache::FMediaSampleCache()
	: CacheAhead(FTimespan::Zero())
	, CacheBehind(FTimespan::Zero())
{ }


/* FMediaSampleCache interface
*****************************************************************************/

void FMediaSampleCache::Empty()
{
	FScopeLock Lock(&CriticalSection);

	AudioSamples.Empty();
	MetadataSamples.Empty();
	OverlaySamples.Empty();
	VideoSamples.Empty();
}


TSharedPtr<IMediaAudioSample, ESPMode::ThreadSafe> FMediaSampleCache::GetAudioSample(FTimespan /*Time*/)
{
	return nullptr;
/*
	FScopeLock Lock(&CriticalSection);

	FetchAudioSamples(Player->GetSamples());

	for (const TSharedPtr<IMediaAudioSample, ESPMode::ThreadSafe>& Sample : AudioSamples)
	{
		const FTimespan SampleTime = Sample->GetTime();

		if ((Time >= SampleTime) && (Time < SampleTime + Sample->GetDuration()))
		{
			return Sample;
		}
	}

#if 1
	UE_LOGF(LogMediaUtils, Log, "---------------------------------------------------------- %d", Time.GetTicks());
	for (const TSharedPtr<IMediaAudioSample, ESPMode::ThreadSafe>& Sample : AudioSamples)
	{
		UE_LOGF(LogMediaUtils, Log, "%d - %d (%d)", Sample->GetTime().GetTicks(), (Sample->GetTime() + Sample->GetDuration()).GetTicks() - 1, Sample->GetFrames());
	}
	UE_LOGF(LogMediaUtils, Log, "----------------------------------------------------------");
#endif

	return nullptr;*/
}


void FMediaSampleCache::GetCachedAudioSampleRanges(TRangeSet<FTimespan>& OutTimeRanges) const
{
	FScopeLock Lock(&CriticalSection);

	for (const TSharedPtr<IMediaAudioSample, ESPMode::ThreadSafe>& Sample : AudioSamples)
	{
		const FTimespan SampleTime = Sample->GetTime().Time;
		OutTimeRanges.Add(TRange<FTimespan>(SampleTime, SampleTime + Sample->GetDuration()));
	}
}


void FMediaSampleCache::GetCachedVideoSampleRanges(TRangeSet<FTimespan>& OutTimeRanges) const
{
	FScopeLock Lock(&CriticalSection);

	for (const TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& Sample : VideoSamples)
	{
		const FTimespan SampleTime = Sample->GetTime().Time;
		OutTimeRanges.Add(TRange<FTimespan>(SampleTime, SampleTime + Sample->GetDuration()));
	}
}


void FMediaSampleCache::GetOverlaySamples(FTimespan Time, TArray<TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe>>& /*OutSamples*/)
{
/*
	FScopeLock Lock(&CriticalSection);

	FetchOverlaySamples(Player->GetSamples());

	for (const TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe>& Sample : OverlaySamples)
	{
		const FTimespan SampleTime = Sample->GetTime();

		if ((Time >= SampleTime) && (Time <= SampleTime + Sample->GetDuration()))
		{
			OutSamples.Add(Sample);
		}
	}*/
}


TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe> FMediaSampleCache::GetVideoSample(FTimespan /*Time*/, bool /*Forward*/)
{
	return nullptr;
/*
	FScopeLock Lock(&CriticalSection);

	FetchVideoSamples(Player->GetSamples());

	TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe> BestMatch;

	if (Forward)
	{
		FTimespan BestTime = FTimespan::MinValue();

		for (const TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& Sample : VideoSamples)
		{
			FTimespan SampleTime = Sample->GetTime();

			if ((SampleTime <= Time) && (SampleTime > BestTime))
			{
				BestMatch = Sample;

				if (SampleTime == Time)
				{
					break; // exact match
				}

				BestTime = SampleTime;
			}
		}
	}
	else
	{
		FTimespan BestTime = FTimespan::MaxValue();

		for (const TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& Sample : VideoSamples)
		{
			FTimespan SampleTime = Sample->GetTime() + Sample->GetDuration();

			if ((SampleTime >= Time) && (SampleTime < BestTime))
			{
				BestMatch = Sample;

				if (SampleTime == Time)
				{
					break; // exact match
				}

				BestTime = SampleTime;
			}
		}
	}

	return BestMatch;*/
}


void FMediaSampleCache::Tick(FTimespan /*DeltaTime*/, float /*Rate*/, FTimespan /*Time*/)
{/*
	// determine cache window
	const float CurrentRate = Player->GetControls().GetRate();
	const FTimespan CurrentTime = Player->GetControls().GetTime();
	const FTimespan Duration = Player->GetControls().GetDuration();

	TRange<FTimespan> CacheWindow;

	if (CurrentRate > 0.0f)
	{
		CacheWindow = TRange<FTimespan>(CurrentTime - CacheBehind - DeltaTime, CurrentTime + CacheAhead);
	}
	else if (CurrentRate < 0.0f)
	{
		CacheWindow = TRange<FTimespan>(CurrentTime - CacheAhead, CurrentTime + CacheBehind + DeltaTime);
	}
	else
	{
		FTimespan CacheMax = FMath::Max(CacheAhead, CacheBehind);
		CacheWindow = TRange<FTimespan>(CurrentTime - CacheMax, CurrentTime + CacheMax);
	}

	FScopeLock Lock(&CriticalSection);

	// fetch unconsumed samples
	IMediaSamples& Samples = Player->GetSamples();

	FetchAudioSamples(Samples);
	FetchMetadataSamples(Samples);
	FetchOverlaySamples(Samples);
	FetchVideoSamples(Samples);

	// remove expired samples
	for (TSampleSet<IMediaAudioSample>::TIterator It(AudioSamples); It; ++It)
	{
		const FTimespan SampleTime = (*It)->GetTime();

		if (!CacheWindow.Contains(SampleTime) &&
			!CacheWindow.Contains(SampleTime + Duration) &&
			!CacheWindow.Contains(SampleTime - Duration))
		{
			It.RemoveCurrent();
		}
	}

	for (TSampleSet<IMediaBinarySample>::TIterator It(MetadataSamples); It; ++It)
	{
		const FTimespan SampleTime = (*It)->GetTime();

		if (!CacheWindow.Contains(SampleTime) &&
			!CacheWindow.Contains(SampleTime + Duration) &&
			!CacheWindow.Contains(SampleTime - Duration))
		{
			It.RemoveCurrent();
		}
	}

	for (TSampleSet<IMediaOverlaySample>::TIterator It(OverlaySamples); It; ++It)
	{
		const FTimespan SampleTime = (*It)->GetTime();

		if (!CacheWindow.Contains(SampleTime) &&
			!CacheWindow.Contains(SampleTime + Duration) &&
			!CacheWindow.Contains(SampleTime - Duration))
		{
			It.RemoveCurrent();
		}
	}

	for (TSampleSet<IMediaTextureSample>::TIterator It(VideoSamples); It; ++It)
	{
		const auto& Sample = *It;

		if (Sample->IsCacheable())
		{
			const FTimespan SampleTime = Sample->GetTime();

			if (CacheWindow.Contains(SampleTime) ||
				CacheWindow.Contains(SampleTime + Duration) ||
				CacheWindow.Contains(SampleTime - Duration))
			{
				continue;
			}
		}

		It.RemoveCurrent();
	}*/
}
