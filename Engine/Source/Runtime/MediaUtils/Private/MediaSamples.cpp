// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaSamples.h"

#include "IMediaAudioSample.h"
#include "IMediaBinarySample.h"
#include "IMediaOverlaySample.h"
#include "IMediaTextureSample.h"

/* Local helpers
*****************************************************************************/

template<typename SampleType, typename SinkType>
bool FetchSample(TMediaSampleQueue<SampleType, SinkType>& SampleQueue, TRange<FTimespan> TimeRange, TSharedPtr<SampleType, ESPMode::ThreadSafe>& OutSample)
{
	TSharedPtr<SampleType, ESPMode::ThreadSafe> Sample;

	if (!SampleQueue.Peek(Sample))
	{
		return false;
	}

	const FTimespan SampleTime = Sample->GetTime().Time;

	if (!TimeRange.Overlaps(TRange<FTimespan>(SampleTime, SampleTime + Sample->GetDuration())))
	{
		return false;
	}

	OutSample = Sample;
	SampleQueue.Pop();

	return true;
}


template<typename SampleType, typename SinkType>
bool FetchSample(TMediaSampleQueue<SampleType, SinkType>& SampleQueue, TRange<FMediaTimeStamp> TimeRange, TSharedPtr<SampleType, ESPMode::ThreadSafe>& OutSample)
{
	TSharedPtr<SampleType, ESPMode::ThreadSafe> Sample;

	if (!SampleQueue.Peek(Sample))
	{
		return false;
	}

	const FMediaTimeStamp SampleTime = Sample->GetTime();

	if (!TimeRange.Overlaps(TRange<FMediaTimeStamp>(SampleTime, SampleTime + Sample->GetDuration())))
	{
		return false;
	}

	OutSample = Sample;
	SampleQueue.Pop();

	return true;
}

/* IMediaSamples interface
*****************************************************************************/

FMediaSamples::FMediaSamples(uint32 InMaxNumberOfQueuedAudioSamples, uint32 InMaxNumberOfQueuedVideoSamples, uint32 InMaxNumberOfQueuedCaptionSamples, uint32 InMaxNumberOfQueuedSubtitlesSamples, uint32 InMaxNumberOfQueuedMetaDataSamples)
	: AudioSampleQueue(InMaxNumberOfQueuedAudioSamples)
	, CaptionSampleQueue(InMaxNumberOfQueuedCaptionSamples)
	, MetadataSampleQueue(InMaxNumberOfQueuedMetaDataSamples)
	, SubtitleSampleQueue(InMaxNumberOfQueuedSubtitlesSamples)
	, VideoSampleQueue(InMaxNumberOfQueuedVideoSamples)
{
}

FMediaSamples::~FMediaSamples() = default;

bool FMediaSamples::FetchAudio(TRange<FTimespan> TimeRange, TSharedPtr<IMediaAudioSample, ESPMode::ThreadSafe>& OutSample)
{
	return FetchSample(AudioSampleQueue, TimeRange, OutSample);
}


bool FMediaSamples::FetchCaption(TRange<FTimespan> TimeRange, TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe>& OutSample)
{
	return FetchSample(CaptionSampleQueue, TimeRange, OutSample);
}


bool FMediaSamples::FetchMetadata(TRange<FTimespan> TimeRange, TSharedPtr<IMediaBinarySample, ESPMode::ThreadSafe>& OutSample)
{
	return FetchSample(MetadataSampleQueue, TimeRange, OutSample);
}


bool FMediaSamples::FetchSubtitle(TRange<FTimespan> TimeRange, TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe>& OutSample)
{
	return FetchSample(SubtitleSampleQueue, TimeRange, OutSample);
}


bool FMediaSamples::FetchVideo(TRange<FTimespan> TimeRange, TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& OutSample)
{
	return FetchSample(VideoSampleQueue, TimeRange, OutSample);
}

bool FMediaSamples::FetchAudio(TRange<FMediaTimeStamp> TimeRange, TSharedPtr<IMediaAudioSample, ESPMode::ThreadSafe>& OutSample)
{
	return FetchSample(AudioSampleQueue, TimeRange, OutSample);
}

bool FMediaSamples::FetchCaption(TRange<FMediaTimeStamp> TimeRange, TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe>& OutSample)
{
	return FetchSample(CaptionSampleQueue, TimeRange, OutSample);
}

bool FMediaSamples::FetchSubtitle(TRange<FMediaTimeStamp> TimeRange, TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe>& OutSample)
{
	return FetchSample(SubtitleSampleQueue, TimeRange, OutSample);
}

bool FMediaSamples::FetchMetadata(TRange<FMediaTimeStamp> TimeRange, TSharedPtr<IMediaBinarySample, ESPMode::ThreadSafe>& OutSample)
{
	return FetchSample(MetadataSampleQueue, TimeRange, OutSample);
}

bool FMediaSamples::FetchVideo(TRange<FMediaTimeStamp> TimeRange, TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& OutSample)
{
	return FetchSample(VideoSampleQueue, TimeRange, OutSample);
}

void FMediaSamples::FlushSamples()
{
	// Flushing may have various side effects that better all happen on the gamethread
	check(IsInGameThread() || IsInSlateThread());

	AudioSampleQueue.RequestFlush();
	MetadataSampleQueue.RequestFlush();
	CaptionSampleQueue.RequestFlush();
	SubtitleSampleQueue.RequestFlush();
	VideoSampleQueue.RequestFlush();
}

void FMediaSamples::SetMinExpectedNextSequenceIndex(TOptional<int32> InNextSequenceIndex)
{
	if (!InNextSequenceIndex.IsSet())
	{
		NextMinSequenceIndex.Reset();
		return;
	}
	if (NextMinSequenceIndex == InNextSequenceIndex)
	{
		return;
	}
	NextMinSequenceIndex = InNextSequenceIndex;
	int32 nsi = InNextSequenceIndex.GetValue();

	AudioSampleQueue.PurgeUntilSequenceIndex(nsi);
	MetadataSampleQueue.PurgeUntilSequenceIndex(nsi);
	CaptionSampleQueue.PurgeUntilSequenceIndex(nsi);
	SubtitleSampleQueue.PurgeUntilSequenceIndex(nsi);
	VideoSampleQueue.PurgeUntilSequenceIndex(nsi);
}

void FMediaSamples::AddAudio(const TSharedRef<IMediaAudioSample, ESPMode::ThreadSafe>& Sample)
{
	if (Sample->GetTime().GetSequenceIndex() >= NextMinSequenceIndex.Get(0))
	{
		AudioSampleQueue.Enqueue(Sample);
	}
}

void FMediaSamples::AddCaption(const TSharedRef<IMediaOverlaySample, ESPMode::ThreadSafe>& Sample)
{
	if (Sample->GetTime().GetSequenceIndex() >= NextMinSequenceIndex.Get(0))
	{
		CaptionSampleQueue.Enqueue(Sample);
	}
}

void FMediaSamples::AddMetadata(const TSharedRef<IMediaBinarySample, ESPMode::ThreadSafe>& Sample)
{
	if (Sample->GetTime().GetSequenceIndex() >= NextMinSequenceIndex.Get(0))
	{
		MetadataSampleQueue.Enqueue(Sample);
	}
}

void FMediaSamples::AddSubtitle(const TSharedRef<IMediaOverlaySample, ESPMode::ThreadSafe>& Sample)
{
	if (Sample->GetTime().GetSequenceIndex() >= NextMinSequenceIndex.Get(0))
	{
		SubtitleSampleQueue.Enqueue(Sample);
	}
}

void FMediaSamples::AddVideo(const TSharedRef<IMediaTextureSample, ESPMode::ThreadSafe>& Sample)
{
	if (Sample->GetTime().GetSequenceIndex() >= NextMinSequenceIndex.Get(0))
	{
		VideoSampleQueue.Enqueue(Sample);
	}
}



/**
 * Fetch video sample best suited for the given time range. Samples prior to the selected one will be removed from the queue.
 */
FMediaSamples::EFetchBestSampleResult FMediaSamples::FetchBestVideoSampleForTimeRange(const TRange<FMediaTimeStamp> & TimeRange, TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& OutSample, bool bReverse, bool bConsistentResult)
{
	EMediaSampleQueueFetchResult FetchRes =	VideoSampleQueue.FetchBestSampleForTimeRange(TimeRange, OutSample, bReverse, bConsistentResult);
	EFetchBestSampleResult Ret = FetchRes == EMediaSampleQueueFetchResult::Found ? EFetchBestSampleResult::Ok :
								 FetchRes == EMediaSampleQueueFetchResult::None ? EFetchBestSampleResult::NoSample : EFetchBestSampleResult::PurgedToEmpty;
	return Ret;
}

/**
 * Remove any video samples from the queue that have no chance of being displayed anymore
 */
uint32 FMediaSamples::PurgeOutdatedVideoSamples(const FMediaTimeStamp & ReferenceTime, bool bReversed, FTimespan MaxAge)
{
	return VideoSampleQueue.PurgeOutdatedSamples(ReferenceTime, bReversed, MaxAge);
}

/**
 * Remove any subtitle samples from the queue that have no chance of being displayed anymore
 */
uint32 FMediaSamples::PurgeOutdatedSubtitleSamples(const FMediaTimeStamp & ReferenceTime, bool bReversed, FTimespan MaxAge)
{
	return SubtitleSampleQueue.PurgeOutdatedSamples(ReferenceTime, bReversed, MaxAge);
}

/**
 * Remove any caption samples from the queue that have no chance of being displayed anymore
 */
uint32 FMediaSamples::PurgeOutdatedCaptionSamples(const FMediaTimeStamp& ReferenceTime, bool bReversed, FTimespan MaxAge)
{
	return CaptionSampleQueue.PurgeOutdatedSamples(ReferenceTime, bReversed, MaxAge);
}

/**
 * Remove any caption samples from the queue that have no chance of being displayed anymore
 */
uint32 FMediaSamples::PurgeOutdatedMetadataSamples(const FMediaTimeStamp& ReferenceTime, bool bReversed, FTimespan MaxAge)
{
	return MetadataSampleQueue.PurgeOutdatedSamples(ReferenceTime, bReversed, MaxAge);
}

/**
 * Check if can receive more video samples
 */
bool FMediaSamples::CanReceiveVideoSamples(uint32 Num) const
{
	return VideoSampleQueue.CanAcceptSamples(Num);
}

/**
 * Check if can receive more audio samples
 */
bool FMediaSamples::CanReceiveAudioSamples(uint32 Num) const
{
	return AudioSampleQueue.CanAcceptSamples(Num);
}

/**
 * Check if can receive more subtitle samples
 */
bool  FMediaSamples::CanReceiveSubtitleSamples(uint32 Num) const
{
	return SubtitleSampleQueue.CanAcceptSamples(Num);
}

/**
 * Check if can receive more caption samples
 */
bool  FMediaSamples::CanReceiveCaptionSamples(uint32 Num) const
{
	return CaptionSampleQueue.CanAcceptSamples(Num);
}

/**
 * Check if can receive more metadata samples
 */
bool  FMediaSamples::CanReceiveMetadataSamples(uint32 Num) const
{
	return MetadataSampleQueue.CanAcceptSamples(Num);
}
