// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaTimeSource.h"
#include "Misc/Timespan.h"


/**
 * Implements the a media time source that derives its time from the application's global time.
 */
class FAppMediaTimeSource
	: public IMediaTimeSource
{
public:

	/** Virtual destructor. */
	virtual ~FAppMediaTimeSource() = default;

public:

	//~ IMediaTimeSource interface

	MEDIAUTILS_API virtual FTimespan GetTimecode() override;
};
