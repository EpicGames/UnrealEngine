// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"

class FMediaPlayerFacade;
class IMediaOptions;
class IMediaPlayerFactory;

struct FMediaPlayerOptions;

/*
	A single IMediaPlayerLifecycleManagerDelegate can be registered with the MediaModule to monitor and control
	player lifecycles throughout the media framework.
	One possible use case motivating the implementation is to control player creation as system resources are being
	monitored. As this task is highly dependent on knowledge the media framework itself often does not have, only
	an application-supplied delegate can be expected to make reasonably good decisions.

	The delegate will receive notifications about major lifecycle events of all players handled by the framework.

	To identify the instances the implementation should not rely on pointer comparisons (the pointers in question
	may be used in reallocations or may be no longer valid for some notifications), but should rather use the
	supplied 64-bit ID, which is uniquely (enough) generated for each created instance.

	Notes:

	- All notification callbacks are issued on the game thread
	- Calls into the system can cause further callbacks to be issued even while the call is ongoing
	- IControl is the interface allowing access to player data and instance as well as submitting OpenRequests as needed
	- IOpenRequest is passed into the OnMediaPlayerOpen() callback when a new player is about to be opened / created
	  It contains all information available about the request so that the code can decided to let the request through, delay or deny it.
	  - To just let it pass through return false from the callback
	  - To delay or deny it return true
	  - Keep a reference to the IControl passed in until you are ready to submit the open request or have decided to deny it
	  - To deny it simply do not submit the request and release all references (this will appear to the rest if the system as a failure to open the player)
	- An InstanceID is available on all callbacks, but be aware that the ID is not yet assigned to an actual player during OnMediaPlayerOpen
	- The OnMediaPlayerOpen can be called for a certain InstanceID more then once if the player is reused without being destroyed first
	  - This can lead to the need to track >1 resource use cases for a single ID (e.g. some players will only release the last set of resources once the new resources are already allocated for a while to avoid stalls)
	- Some callbacks may be issued at a time that the player facade associated with the player (and may be even the low level player) do no longer exist. Be prepared for e.g. GetFacade() to return a null reference

	OnMediaPlayerOpen

	Called when a player is about to be opened. If this is the first time the facade receives an open call or if the player type changed this is called before the actual creation of the player.
	In any case this is called early enough to still stop the open from proceeding. Aside of tracking player activity, the ability to control the processing of open calls is the main purpose of this callback.

	OnMediaPlayerCreated

	Called when the creation of a new player succeeded. Note that the open may still fail if the player later on has issues playing the content it is pointed to.

	OnMediaPlayerCreateFailed

	Creation of the player failed. This will posisbly be called from within the SubmitOpenRequest() call of the IControl interface. It signifies that either the player factory did not manage to create a player
	or that the low level player failed to open in the synchronous part of its opening process.

	OnMediaPlayerClosed

	The low level player closed the current playback session.

	OnMediaPlayerDestroyed

	The low level player instance just got destroyed.

	OnMediaPlayerResourcesReleased

	The low level player instance release resources. This callback receives a bitmask of possible resource types (Resource_Flag...) to allow proper tracking of players that release certain types of resources at different times.
	Some players will issue this callback asynchronously, it may hence be triggered long (100-200ms are possible) after the closing or destruction of a player.

*/
//! Interface to receive global player lifetime events from media framework
class IMediaPlayerLifecycleManagerDelegate
{
public:
	enum {
		ResourceFlags_Decoder = 1 << 0,
		ResourceFlags_OutputBuffers = 1 << 1,

		ResourceFlags_All = (1 << 2) - 1,
		ResourceFlags_Any = ResourceFlags_All,
	};

	//! Request to create and open a player
	class IOpenRequest
	{
	public:
		virtual ~IOpenRequest() = default;

		virtual const FString& GetUrl() const = 0;
		virtual const IMediaOptions* GetOptions() const = 0;
		virtual const FMediaPlayerOptions* GetPlayerOptions() const = 0;
		virtual IMediaPlayerFactory* GetPlayerFactory() const = 0;
		virtual bool WillCreateNewPlayer() const = 0;
		virtual bool WillUseNewResources(uint32 ResourceFlags) const = 0;
	};
	typedef TSharedPtr<IOpenRequest, ESPMode::ThreadSafe> IOpenRequestRef;

	//! Control interface for lifecycle delegate
	class IControl
	{
	public:
		virtual ~IControl() = default;

		virtual bool SubmitOpenRequest(IOpenRequestRef && OpenRequest) = 0;

		virtual TSharedPtr<FMediaPlayerFacade, ESPMode::ThreadSafe> GetFacade() const = 0;
		virtual uint64 GetMediaPlayerInstanceID() const = 0;
	};
	typedef TSharedPtr<IControl, ESPMode::ThreadSafe> IControlRef;

	virtual ~IMediaPlayerLifecycleManagerDelegate() = default;

	virtual bool OnMediaPlayerOpen(IControlRef Control, IOpenRequestRef OpenRequest) = 0;
	virtual void OnMediaPlayerCreated(IControlRef Control) = 0;
	virtual void OnMediaPlayerCreateFailed(IControlRef Control) = 0;
	virtual void OnMediaPlayerClosed(IControlRef Control) = 0;
	virtual void OnMediaPlayerDestroyed(IControlRef Control) = 0;
	virtual void OnMediaPlayerResourcesReleased(IControlRef Control, uint32 ResourceFlags) = 0;
};
