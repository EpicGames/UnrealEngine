// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "IMediaTimeSource.h"
#include "MediaSampleSource.h"
#include "Misc/Timespan.h"
#include "Templates/SharedPointer.h"

class IMediaAudioSample;


/**
 *
 */
class FMediaAudioResampler
{
public:

	/** Default constructor. */
	MEDIAUTILS_API FMediaAudioResampler();

public:

	/** Flush the resampler. */
	MEDIAUTILS_API void Flush();

	/**
	 * Generate the next frame of audio.
	 *
	 * @param Output The output sample buffer (will be incremented by the number of sample values written).
	 * @param OutTime The time of the first output frame that was returned.
	 * @param FramesRequested The maximum number of frames to get.
	 * @param Rate The current play rate.
	 * @param Time The current play time.
	 * @param SampleSource The object that provides more audio samples if needed.
	 * @param JumpFrame Frame that a jump in time occured on.
	 * @return The actual number of frames returned.
	 */
	MEDIAUTILS_API uint32 Generate(float* Output, FMediaTimeStamp& OutTime, const uint32 FramesRequested, float Rate, FTimespan Time, FMediaAudioSampleSource& SampleSource, uint32& JumpFrame);

	/**
	 * Initialize the resampler.
	 *
	 * @param InOutputSampleRate Desired output sample rate.
	 * @see SetAudioSamples
	 */
	MEDIAUTILS_API void Initialize(const uint32 InOutputChannels, const uint32 InOutputSampleRate);

protected:

	/**
	 * Clear the input samples.
	 *
	 * @see SetInput
	 */
	MEDIAUTILS_API void ClearInput();

	/**
	 * Set the audio sample to be resampled.
	 *
	 * @param Sample The input audio sample.
	 * @return true on success, false otherwise.
	 * @see ClearInput
	 */
	MEDIAUTILS_API bool SetInput(const TSharedPtr<IMediaAudioSample, ESPMode::ThreadSafe>& Sample);

private:

	//~ Frame interpolation values

	/** Cached sample values of the current frame. */
	float CurrentFrame[8];

	/** Cached sample values of the next frame. */
	float NextFrame[8];

	/** Linear interpolation between current and next frame. */
	float FrameAlpha;

	/** Index of the current input read position. */
	int64 FrameIndex;

	/** Index of the previously generated frame (to avoid calculating it again). */
	int64 LastFrameIndex;

	/** The play rate of the previously generated frame. */
	float LastRate;

private:

	//~ Input and output specs

	/** The input buffer. */
	TArray<float> Input;

	/** Duration of the input buffer. */
	FTimespan InputDuration;

	/** Number of frames in input buffer. */
	uint32 InputFrames;

	/** Sample rate of input buffer. */
	uint32 InputSampleRate;

	/** Start time of the input buffer. */
	FMediaTimeStamp InputTime;

	/** Number of channels in the output. */
	uint32 OutputChannels;

	/** Sample rate of the output. */
	uint32 OutputSampleRate;
};
