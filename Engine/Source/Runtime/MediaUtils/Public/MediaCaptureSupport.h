// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "IMediaCaptureSupport.h"

struct FMediaCaptureDeviceInfo;


namespace MediaCaptureSupport
{
	/**
	 * Enumerate available audio capture devices.
	 *
	 * @param OutDeviceInfos Will contain information about the devices.
	 * @see EnumerateVideoCaptureDevices
	 */
	MEDIAUTILS_API void EnumerateAudioCaptureDevices(TArray<FMediaCaptureDeviceInfo>& OutDeviceInfos);

	/**
	 * Enumerate available video capture devices.
	 *
	 * @param OutDeviceInfos Will contain information about the devices.
	 * @see EnumerateAudioCaptureDevices
	 */
	MEDIAUTILS_API void EnumerateVideoCaptureDevices(TArray<FMediaCaptureDeviceInfo>& OutDeviceInfos);
}
