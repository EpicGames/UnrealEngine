// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Misc/Build.h"
#include "Templates/SharedPointer.h"

#define UE_MEDIAUTILS_DEVELOPMENT_DELEGATE (UE_BUILD_DEVELOPMENT || UE_BUILD_DEBUG)

class FRHITexture;
class FTextureResource;
class IMediaTextureSample;
class UMediaTexture;

class FMediaDelegates
{
public:
	using TSharedMediaTextureSample = TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSampleDiscardedDelegate, const UMediaTexture* /*Owner*/, TSharedMediaTextureSample /*FlushedSample*/);
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnPreSampleRenderDelegate, const UMediaTexture* /*Owner*/, bool /*UseSample*/, TSharedMediaTextureSample /*SampleToRender*/);
	DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FOnSampleRenderedDelegate, const FTextureResource* /*MediaTextureResource*/, TSharedMediaTextureSample /*RenderedSample*/);
	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnMediaTextureResourceReleasedDelegate, FRHITexture* /*ReleasedIntermediateTarget*/);

#if UE_MEDIAUTILS_DEVELOPMENT_DELEGATE

	/**
	 * Callback when a sample is discarded from the MediaTexture renderer.
	 * Used to debug the samples queue. Only available in a development build.
	 * @see UE_MEDIAUTILS_DEVELOPMENT_DELEGATE
	 */
	static MEDIAUTILS_API FOnSampleDiscardedDelegate OnSampleDiscarded_RenderThread;

	/**
	 * Callback when a sample is going to be used to update the MediaTexture.
	 * Used to debug the samples queue. Only available in a development build.
	 * @see UE_MEDIAUTILS_DEVELOPMENT_DELEGATE
	 */
	static MEDIAUTILS_API FOnPreSampleRenderDelegate OnPreSampleRender_RenderThread;

#endif //UE_MEDIAUTILS_DEVELOPMENT_DELEGATE

	/**
	 * Callback when a sample has been rendered in the MediaTexture.
	 * This can be used to extend the MediaTexture rendering pipeline.
	 */
	static MEDIAUTILS_API FOnSampleRenderedDelegate OnSampleRendered_RenderThread;

	/**
	 * Fires from FMediaTextureResource::ReleaseRHI immediately before the intermediate
	 * render target's RHI reference is dropped. Subscribers receive the FRHITexture*
	 * while it is still valid and must not retain it past the callback. Lets a subscriber
	 * drop per-texture GPU state while the underlying resource is alive (e.g. unmap
	 * reserved-resource tiles), avoiding stale-pointer dereference during process-exit
	 * drain.
	 */
	static MEDIAUTILS_API FOnMediaTextureResourceReleasedDelegate OnMediaTextureResourceReleased_RenderThread;
};
