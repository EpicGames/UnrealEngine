// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

enum class EMediaEvent;
enum class EMediaState;
enum class EMediaTrackType;


namespace MediaUtils
{
	/**
	 * Convert a media event to a human readable string.
	 *
	 * @param Event The event.
	 * @return The corresponding string.
	 * @see StateToString, TrackTypeToString
	 */
	MEDIAUTILS_API FString EventToString(EMediaEvent Event);

	/**
	 * Convert a media state to a human readable string.
	 *
	 * @param State The state.
	 * @return The corresponding string.
	 * @see EventToString, TrackTypeToString
	 */
	MEDIAUTILS_API FString StateToString(EMediaState State);

	/**
	 * Convert a media track type to a human readable string.
	 *
	 * @param TrackType The track type.
	 * @return The corresponding string.
	 * @see EventToString, StateToString
	 */
	MEDIAUTILS_API FString TrackTypeToString(EMediaTrackType TrackType);
}
