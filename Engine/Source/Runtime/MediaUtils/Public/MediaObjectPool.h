// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Queue.h"
#include "Misc/ScopeLock.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"


/**
 * Interface for objects that can be pooled.
 */
class IMediaPoolable
{
public:

	/**
	 * Called when the object is removed from the pool.
	 *
	 * Override this method to initialize a poolable object
	 * before it is being reused.
	 *
	 * @see ShutdownPoolable
	 */
	virtual void InitializePoolable() { }

	/**
	 * Called when the object added to the pool.
	 *
	 * Override this method to clean up a poolable object
	 * when it is no longer used.
	 *
	 * @see InitializePoolable
	 */
	virtual void ShutdownPoolable() { }

	/**
	 * Used to check if returned object is ready for reuse right away
	 */
	virtual bool IsReadyForReuse()
	{
		return true;
	}

public:

	/** Virtual destructor. */
	virtual ~IMediaPoolable() = default;
};


template<typename ObjectType> class TMediaPoolDefaultObjectAllocator
{
public:
	static ObjectType *Alloc() { return new ObjectType; }
};

/**
 * Template for media object pools.
 *
 * Poolable objects are required to implement the IMediaPoolable interface.
 *
 * @param ObjectType The type of objects managed by this pool. 
 * @todo gmp: Make media object pool lock-free
 */
template<typename ObjectType, typename ObjectAllocator=TMediaPoolDefaultObjectAllocator<ObjectType>>
class TMediaObjectPool
{
	static_assert(TPointerIsConvertibleFromTo<ObjectType, IMediaPoolable>::Value, "Poolable objects must implement the IMediaPoolable interface.");

	/** Object pool storage. */
	class TStorage
	{
	public:
		TStorage(ObjectAllocator *InObjectAllocatorInstance)
			: ObjectAllocatorInstance(InObjectAllocatorInstance)
		{}

		/** Destructor. */
		~TStorage()
		{
			Reserve(0);
			ObjectType* Object;
			while (WaitReadyForReuse.Dequeue(Object))
			{
				Object->ShutdownPoolable();
				delete Object;
			}
		}

	public:

		/** 
		* Acquire an object from the pool. 
		* bAlloc allows to chose if new object should be allocated and added to the pool.
		*/
		ObjectType* Acquire(bool bAlloc = true)
		{
			ObjectType* Result = nullptr;
			{
				FScopeLock Lock(&CriticalSection);

				if (Pool.Num() > 0)
				{
					Result = Pool.Pop(EAllowShrinking::No);
				}
				else
				{
					// Check for objects to ready to enter the pool & grab the first we find as our result...
					// (we move all into the pool, we can to possibly safe on resources allocated by the objects)
					ObjectType* PeekObject;
					while (WaitReadyForReuse.Peek(PeekObject))
					{
						if (!PeekObject->IsReadyForReuse())
						{
							break;
						}

						WaitReadyForReuse.Pop();
						PeekObject->ShutdownPoolable();
						if (!Result)
						{
							Result = PeekObject;
						}
						else
						{
							Pool.Add(PeekObject);
						}
					}
				}
			}

			if (Result == nullptr && bAlloc)
			{
				Result = ObjectAllocatorInstance->Alloc();
			}
			else
			{
				return Result;
			}
			
			Result->InitializePoolable();

			return Result;
		}

		/** Get the number of objects stored. */
		int32 Num() const
		{
			FScopeLock Lock(&CriticalSection);
			return Pool.Num();
		}

		/** Return the given object to the pool. */
		void Release(ObjectType* Object)
		{
			if (Object == nullptr)
			{
				return;
			}

			FScopeLock Lock(&CriticalSection);

			if (Object->IsReadyForReuse())
			{
				Object->ShutdownPoolable();
				Pool.Push(Object);
			}
			else
			{
				WaitReadyForReuse.Enqueue(Object);
			}
		}

		/** Reserve the specified number of objects. */
		void Reserve(uint32 NumObjects)
		{
			FScopeLock Lock(&CriticalSection);

			while (NumObjects < (uint32)Pool.Num())
			{
				delete Pool.Pop(EAllowShrinking::No);
			}

			while (NumObjects > (uint32)Pool.Num())
			{
				Pool.Push(ObjectAllocatorInstance->Alloc());
			}
		}

		/** Regular tick call */
		void Tick()
		{
			if (WaitReadyForReuse.IsEmpty())
			{
				// Conservative early out to avoid CS: we will get any items missed the next time around
				return;
			}

			FScopeLock Lock(&CriticalSection);

			ObjectType* Object;
			while (WaitReadyForReuse.Peek(Object))
			{
				if (!Object->IsReadyForReuse())
				{
					break;
				}

				Object->ShutdownPoolable();
				Pool.Push(Object);
				WaitReadyForReuse.Pop();
			}
		}

	private:

		/** Critical section for synchronizing access to the free list. */
		mutable FCriticalSection CriticalSection;

		/** List of unused objects. */
		TArray<ObjectType*> Pool;

		/** List of unused objects, waiting for reuse-ability. */
		TQueue<ObjectType*> WaitReadyForReuse;

		/** Object allocator instance (nullptr by default) */
		ObjectAllocator *ObjectAllocatorInstance;
	};


	/** Deleter for pooled objects. */
	class TDeleter
	{
	public:

		/** Create and initialize a new instance. */
		TDeleter(const TSharedRef<TStorage, ESPMode::ThreadSafe>& InStorage)
			: StoragePtr(InStorage)
		{ }

		/** Function operator to execute deleter. */
		void operator()(ObjectType* ObjectToDelete)
		{
			TSharedPtr<TStorage, ESPMode::ThreadSafe> PinnedStorage = StoragePtr.Pin();

			if (PinnedStorage.IsValid())
			{
				PinnedStorage->Release(ObjectToDelete);
			}
			else
			{
				delete ObjectToDelete;
			}
		}

	private:

		/** Weak pointer to object pool storage. */
		TWeakPtr<TStorage, ESPMode::ThreadSafe> StoragePtr;
	};

public:

	/** Default constructor. */
	TMediaObjectPool(ObjectAllocator *ObjectAllocatorInstance = nullptr)
		: Storage(MakeShareable(new TStorage(ObjectAllocatorInstance)))
	{ }

	/**
	 * Create and initialize a new instance.
	 *
	 * @param NumReserve Number of objects to reserve.
	 */
	TMediaObjectPool(uint32 NumReserve)
		: Storage(MakeShareable(new TStorage(nullptr)))
	{
		Storage->Reserve(NumReserve);
	}

public:

	/**
	 * Acquire an untracked object from the pool.
	 *
	 * Use the Release method to return the object to the pool.
	 * You can use the ToShared and ToUnique methods to convert
	 * this object to a tracked shared object later if desired.
	 * bAlloc allows to chose if new object should be allocated and added to the pool.
	 *
	 * @return The object.
	 * @see AcquireShared, AcquireUnique, Release, ToShared, ToUnique
	 */
	ObjectType* Acquire(bool bAlloc = true)
	{
		return Storage->Acquire(bAlloc);
	}

	/**
	 * Acquire a shared object from the pool.
	 *
	 * Shared objects do not need to be returned to the pool. They'll be
	 * reclaimed automatically when their reference count goes to zero.
	 *
	 * @return The shared object.
	 * @see Acquire, AcquireUnique, Reset, ToShared
	 */
	TSharedRef<ObjectType, ESPMode::ThreadSafe> AcquireShared()
	{
		ObjectType* Object = Acquire(true);
		check(Object != nullptr);
		return MakeShareable(Object, TDeleter(Storage));
	}

	/**
	 * Acquire a shared object from the pool.
	 *
	 * Shared objects do not need to be returned to the pool. They'll be
	 * reclaimed automatically when their reference count goes to zero.
	 * bAlloc allows to chose if new object should be allocated and added to the pool.
	 *
	 * @return The shared object.
	 * @see Acquire, AcquireUnique, Reset, ToShared
	 */
	TSharedPtr<ObjectType, ESPMode::ThreadSafe> AcquireShared(bool bAlloc)
	{
		ObjectType* Object = Acquire(bAlloc);
		if (!Object && !bAlloc)
		{
			return nullptr;
		}
		check(Object != nullptr);

		return MakeShareable(Object, TDeleter(Storage));
	}

	/**
	 * Get the number of objects available in the pool.
	 *
	 * @return Number of available objects.
	 */
	int32 Num() const
	{
		return Storage->Num();
	}

	/**
	 * Convert an object to a shared pooled object.
	 *
	 * @param Object The object to convert.
	 * @return The shared object.
	 * @see AcquireShared, ToUnique
	 */
	TSharedRef<ObjectType, ESPMode::ThreadSafe> ToShared(ObjectType* Object)
	{
		return MakeShareable(Object, TDeleter(Storage));
	}

	/**
	 * Return the given object to the pool.
	 *
	 * This method can return plain old C++ objects to the pool.
	 * Do not use this method with objects acquired via AcquireShared
	 * or AcquireUnique, because those are returned automatically.
	 *
	 * @see Acquire
	 */
	void Release(ObjectType* Object)
	{
		Storage->Release(Object);
	}

	/**
	 * Reset the pool and reserve a specified number of objects.
	 *
	 * @param NumObjects Number of objects to reserve (default = 0).
	 * @see GetObject
	 */
	void Reset(uint32 NumObjects = 0)
	{
		Storage->Reserve(NumObjects);
	}

	/**
	 * Regular tick call
	 */
	void Tick()
	{
		Storage->Tick();
	}

private:

	/** Storage for pooled objects. */
	TSharedRef<TStorage, ESPMode::ThreadSafe> Storage;
};
