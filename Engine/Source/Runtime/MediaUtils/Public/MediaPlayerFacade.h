// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Queue.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "HAL/CriticalSection.h"
#include "IMediaClockSink.h"
#include "IMediaEventSink.h"
#include "IMediaPlayerLifecycleManager.h"
#include "IMediaTickable.h"
#include "IMediaTimeSource.h"
#include "IMediaTracks.h"
#include "Internationalization/Text.h"
#include "Math/MathFwd.h"
#include "Math/Quat.h"
#include "Math/Range.h"
#include "Math/RangeSet.h"
#include "Math/Rotator.h"
#include "MediaPlayerOptions.h"
#include "MediaSampleSink.h"
#include "MediaSampleSinks.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "Misc/Timespan.h"
#include "Misc/Timecode.h"
#include "Misc/Variant.h"
#include "Templates/Atomic.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FMediaSampleCache;
class IMediaModule;
class IMediaOptions;
class IMediaPlayer;
class IMediaPlayerFactory;
class IMediaSamples;
class IMediaMetadataItem;

enum class EMediaEvent;
enum class EMediaCacheState;
enum class EMediaThreads;
enum class EMediaTrackType;
enum class EMediaTimeRangeType;

struct FMediaAudioTrackFormat;
struct FMediaPlayerOptions;
struct FMediaVideoTrackFormat;


/**
 * Facade for low-level media player objects.
 *
 * The purpose of this class is to provide a simpler interface to low-level media player
 * implementations. It implements common functionality, such as translating between time
 * codes and play times, and manages the selection and creation of player implementations
 * for a given media source.
 *
 * Note that, unlike the low-level methods in IMediaTracks, most track and track format
 * related methods in this class allow for INDEX_NONE to be used as track and format
 * indices in order to indicate the 'current selection'.
 */
class FMediaPlayerFacade
	: public IMediaClockSink
	, public IMediaTickable
	, protected IMediaEventSink
	, public TSharedFromThis<FMediaPlayerFacade, ESPMode::ThreadSafe>
{
public:

	/** Name of the desired native player, if any. */
	FName DesiredPlayerName;

	/** Extra time to reduce from current player's time. */
	FTimespan TimeDelay;

	/** Active media player options. */
	TOptional<FMediaPlayerOptions> ActivePlayerOptions;

public:

	/** Default constructor. */
	MEDIAUTILS_API FMediaPlayerFacade(TWeakObjectPtr<UMediaPlayer> InMediaPlayer);

	/** Virtual destructor. */
	MEDIAUTILS_API virtual ~FMediaPlayerFacade();

public:

	/**
	 * Add the given audio sample sink to this player.
	 *
	 * @param SampleSink The sink to receive audio samples.
	 * @see  AddCaptionSampleSink, AddMetadataSampleSink, AddSubtitleSampleSink, AddVideoSampleSink
	 */
	MEDIAUTILS_API void AddAudioSampleSink(const TSharedRef<FMediaAudioSampleSink, ESPMode::ThreadSafe>& SampleSink);

	/**
	 * Add the given audio sample sink to this player.
	 *
	 * @param SampleSink The sink to receive caption samples.
	 * @see AddAudioSampleSink, AddMetadataSampleSink, AddSubtitleSampleSink, AddVideoSampleSink
	 */
	MEDIAUTILS_API void AddCaptionSampleSink(const TSharedRef<FMediaOverlaySampleSink, ESPMode::ThreadSafe>& SampleSink);

	/**
	 * Add the given audio sample sink to this player.
	 *
	 * @param SampleSink The sink to receive metadata samples.
	 * @see AddAudioSampleSink, AddCaptionSampleSink, AddSubtitleSampleSink, AddVideoSampleSink
	 */
	MEDIAUTILS_API void AddMetadataSampleSink(const TSharedRef<FMediaBinarySampleSink, ESPMode::ThreadSafe>& SampleSink);

	/**
	 * Add the given audio sample sink to this player.
	 *
	 * @param SampleSink The sink to receive subtitle samples.
	 * @see AddAudioSampleSink, AddCaptionSampleSink, AddMetadataSampleSink, AddVideoSampleSink
	 */
	MEDIAUTILS_API void AddSubtitleSampleSink(const TSharedRef<FMediaOverlaySampleSink, ESPMode::ThreadSafe>& SampleSink);

	/**
	 * Add the given audio sample sink to this player.
	 *
	 * @param SampleSink The sink to receive video samples.
	 * @see AddAudioSampleSink, AddCaptionSampleSink, AddMetadataSampleSink, AddSubtitleSampleSink
	 */
	MEDIAUTILS_API void AddVideoSampleSink(const TSharedRef<FMediaTextureSampleSink, ESPMode::ThreadSafe>& SampleSink);

	/**
	 * Whether playback can be paused.
	 *
	 * Playback can be paused if the media supports pausing
	 * and if it is currently playing.
	 *
	 * @return true if pausing is allowed, false otherwise.
	 * @see CanResume, CanScrub, CanSeek, Pause
	 */
	MEDIAUTILS_API bool CanPause() const;

	/**
	 * Whether the specified URL can be played by this player.
	 *
	 * If a desired player name is set for this player, it will only check
	 * whether that particular player type can play the specified URL.
	 *
	 * @param Url The URL to check.
	 * @param Options Optional media parameters.
	 * @see CanPlaySource, SetDesiredPlayerName
	 */
	MEDIAUTILS_API bool CanPlayUrl(const FString& Url, const IMediaOptions* Options);

	/**
	 * Whether playback can be resumed.
	 *
	 * Playback can be resumed if media is loaded and
	 * if it is not already playing.
	 *
	 * @return true if resuming is allowed, false otherwise.
	 * @see CanPause, CanScrub, CanSeek, SetRate
	 */
	MEDIAUTILS_API bool CanResume() const;

	/**
	 * Whether playback can be scrubbed.
	 *
	 * @return true if scrubbing is allowed, false otherwise.
	 * @see CanPause, CanResume, CanSeek, Seek
	 */
	MEDIAUTILS_API bool CanScrub() const;

	/**
	 * Whether playback can jump to a position.
	 *
	 * @return true if seeking is allowed, false otherwise.
	 * @see CanPause, CanResume, CanScrub, Seek
	 */
	MEDIAUTILS_API bool CanSeek() const;

	/**
	 * Check whether the player supports playing back of range within the media.
	 *
	 * @return true if playing back a range is supported, false otherwise.
	 * @see GetPlaybackTimeRange, SetPlaybackTimeRange
	 */
	MEDIAUTILS_API bool SupportsPlaybackTimeRange() const;

	/**
	 * Close the currently open media, if any.
	 */
	MEDIAUTILS_API void Close();

	/**
	 * Get the number of channels in the specified audio track.
	 *
	 * @param TrackIndex Index of the audio track.
	 * @param FormatIndex Index of the track format.
	 * @return Number of channels.
	 * @see GetAudioTrackSampleRate, GetAudioTrackType
	 */
	MEDIAUTILS_API uint32 GetAudioTrackChannels(int32 TrackIndex, int32 FormatIndex) const;

	/**
	 * Get the sample rate of the specified audio track.
	 *
	 * @param TrackIndex Index of the audio track.
	 * @param FormatIndex Index of the track format.
	 * @return Samples per second.
	 * @see GetAudioTrackChannels, GetAudioTrackType
	 */
	MEDIAUTILS_API uint32 GetAudioTrackSampleRate(int32 TrackIndex, int32 FormatIndex) const;

	/**
	 * Get the type of the specified audio track format.
	 *
	 * @param TrackIndex The index of the track.
	 * @param FormatIndex Index of the track format.
	 * @return Audio format type string.
	 * @see GetAudioTrackSampleRate, GetAudioTrackSampleRate
	 */
	MEDIAUTILS_API FString GetAudioTrackType(int32 TrackIndex, int32 FormatIndex) const;

	/**
	 * Get the media's duration.
	 *
	 * @return A time span representing the duration.
	 * @see GetTime, Seek
	 */
	MEDIAUTILS_API FTimespan GetDuration() const;

	/**
	 * Get the player's globally unique identifier.
	 *
	 * @return The Guid.
	 * @see SetGuid
	 */
	MEDIAUTILS_API const FGuid& GetGuid();

	/**
	 * Get debug information about the player and currently opened media.
	 *
	 * @return Information string.
	 * @see GetStats
	 */
	MEDIAUTILS_API FString GetInfo() const;

	/**
	 * Get information about the media that is playing.
	 *
	 * @param	InfoName		Name of the information we want.
	 * @returns					Requested information, or empty if not available.
	 * @see						UMediaPlayer::GetMediaInfo.
	 */
	MEDIAUTILS_API FVariant GetMediaInfo(FName InfoName) const;

	/**
	 * Get the human readable name of the currently loaded media source.
	 *
	 * @return Media source name, or empty text if no media is opened
	 * @see GetPlayerName, GetUrl
	 */
	MEDIAUTILS_API FText GetMediaName() const;

	/**
	 * Get meta data contained in the current stream
	 *
	 * @return Map with arrays of IMediaMetaDataItem entries describing any metadata found in the current stream
	 * @note Listen to EMediaEvent::MetadataChanged to catch updates to this data
	 */
	MEDIAUTILS_API TSharedPtr<TMap<FString, TArray<TUniquePtr<IMediaMetadataItem>>>, ESPMode::ThreadSafe> GetMediaMetadata() const;

	/**
	 * Get the number of tracks of the given type.
	 *
	 * @param TrackType The type of media tracks.
	 * @return Number of tracks.
	 * @see GetSelectedTrack, SelectTrack
	 */
	MEDIAUTILS_API int32 GetNumTracks(EMediaTrackType TrackType) const;

	/**
	 * Get the number of formats of the specified track.
	 *
	 * @param TrackType The type of media tracks.
	 * @param TrackIndex The index of the track.
	 * @return Number of formats.
	 * @see GetNumTracks, GetSelectedTrack, SelectTrack
	 */
	MEDIAUTILS_API int32 GetNumTrackFormats(EMediaTrackType TrackType, int32 TrackIndex) const;

	/**
	 * Get the low-level player associated with this object.
	 *
	 * @return The player, or nullptr if no player was created.
	 */
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> GetPlayer() const
	{
		return Player;
	}

	/**
	 * Get the name of the current native media player.
	 *
	 * @return Player name, or NAME_None if not available.
	 * @see GetMediaName
	 */
	MEDIAUTILS_API FName GetPlayerName() const;

	/**
	 * Get the media's current playback rate.
	 *
	 * @return The playback rate.
	 * @see SetRate, SupportsRate
	 */
	MEDIAUTILS_API float GetRate() const;

	/**
	 * Get the index of the currently selected track of the given type.
	 *
	 * @param TrackType The type of track to get.
	 * @return The index of the selected track, or INDEX_NONE if no track is active.
	 * @see GetNumTracks, SelectTrack
	 */
	MEDIAUTILS_API int32 GetSelectedTrack(EMediaTrackType TrackType) const;

	/**
	 * Get playback statistics information.
	 *
	 * @return Information string.
	 * @see GetInfo
	 */
	MEDIAUTILS_API FString GetStats() const;

	/**
	 * Get the supported playback rates.
	 *
	 * @param Unthinned Whether the rates are for unthinned playback (default = true).
	 * @return The ranges of supported rates.
	 * @see SetRate, SupportsRate
	 */
	MEDIAUTILS_API TRangeSet<float> GetSupportedRates(bool Unthinned = true) const;

	/**
	 * Returns the current playback range of the media.
	 * If playing back a range is not supported, the range returned will be equal
	 * to [ 0, GetDuration() ).
	 * The media may have an implicit default range provided by the container format
	 * or other means without having called SetPlaybackTimeRange().
	 * The media may have internal time values not starting at 0, which are
	 * conveyed by the range.
	 * Since the range may be only a portion of the media, the duration of the
	 * returned range may be less than the media overall duration returned by
	 * GetDuration().
	 * For live video streams the range may change dynamically as new content
	 * becomes available and old content falls off the timeline.
	 *
	 * @param InRangeToGet The type of range to get.
	 *                     `Absolute` returns the media's smallest and largest timeline values.
	 *                       Unless continuously changing in a Live stream this is usually the
	 *                       same as [ 0, GetDuration() ]. The base time does not have to be
	 *                       zero though.
	 *                     `Current` returns the currently set range, which is a subset of the
	 *                       absolute range.
	 * @return The playback range as queried for or an empty range if there is no open player.
	 * @see SupportsPlaybackTimeRange, SetPlaybackTimeRange, GetDuration
	 */
	MEDIAUTILS_API TRange<FTimespan> GetPlaybackTimeRange(EMediaTimeRangeType InRangeToGet) const;

	/**
	 * Get the media's current playback time.
	 *
	 * @return Playback time.
	 * @see GetDuration, Seek
	 */
	MEDIAUTILS_API FTimespan GetTime() const;

	/**
	 * Get the media's current playback time stamp.
	 *
	 * @return Playback time stamp.
	 */
	MEDIAUTILS_API FMediaTimeStamp GetTimeStamp() const;

	/**
	 * Get the media's current playback time stamp in a "display" version
	 *
	 * @return Playback time stamp.
	 *
	 * @note The timestamp returned here will reflect a user-logic oriented version.
	 *       (e.g. during seeks this will return the seek target rather than the last valid frame still displayed)
	 */
	MEDIAUTILS_API FMediaTimeStamp GetDisplayTimeStamp() const;

	/**
	 * Get the timecode from the video track, if one is available.
	 * If available, the returned value will be the timecode of the sample most recently
	 * sent into the video sink.
	 *
	 * @return Most recently delivered video timecode from the video track.
	 *
	 * @note Even if timecode is available, the value returned reflects the timecode
	 *       that most recently delivered to the video sink. During a seek operation
	 *       the timecode will be unset.
	 */
	MEDIAUTILS_API TOptional<FTimecode> GetVideoTimecode() const;

	/**
	 *	Returns the last video sample time range that has been enqueued in the video sink.
	 */
	MEDIAUTILS_API TRange<FMediaTimeStamp> GetLastProcessedVideoSampleTimeRange() const;

	/**
	 * Get the human readable name of the specified track.
	 *
	 * @param TrackType The type of track.
	 * @param TrackIndex The index of the track.
	 * @return Display name.
	 * @see GetNumTracks, GetTrackLanguage
	 */
	MEDIAUTILS_API FText GetTrackDisplayName(EMediaTrackType TrackType, int32 TrackIndex) const;

	/**
	 * Get the index of the active format of the specified track.
	 *
	 * @param TrackType The type of track.
	 * @param TrackIndex The index of the track.
	 * @return The index of the selected format.
	 * @see GetNumTrackFormats, GetSelectedTrack, SetTrackFormat
	 */
	MEDIAUTILS_API int32 GetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex) const;

	/**
	 * Get the language tag of the specified track.
	 *
	 * @param TrackType The type of track.
	 * @param TrackIndex The index of the track.
	 * @return Language tag, i.e. "en-US" for English, or "und" for undefined.
	 * @see GetNumTracks, GetTrackDisplayName
	 */
	MEDIAUTILS_API FString GetTrackLanguage(EMediaTrackType TrackType, int32 TrackIndex) const;

	/**
	 * Gets the duration of a specific track, if available.
	 * Availability depends on the media format and player implementation.
	 * If unset only the overall media duration can be used.
	 *
	 * @return An optional time span representing the duration.
	 * @see GetDuration
	 */
	MEDIAUTILS_API TOptional<FTimespan> GetTrackDuration(EMediaTrackType TrackType, int32 TrackIndex) const;

	/**
	 * Get the URL of the currently loaded media, if any.
	 *
	 * @return Media URL, or empty string if no media was loaded.
	 */
	const FString& GetUrl() const
	{
		return CurrentUrl;
	}

	/**
	 * Get the aspect ratio of the specified video track.
	 *
	 * @param TrackIndex The index of the track.
	 * @param FormatIndex Index of the track format.
	 * @return Aspect ratio.
	 * @see GetVideoTrackDimensions, GetVideoTrackFrameRate, GetVideoTrackFrameRates, GetVideoTrackType
	 */
	MEDIAUTILS_API float GetVideoTrackAspectRatio(int32 TrackIndex, int32 FormatIndex) const;

	/**
	 * Get the width and height of the specified video track.
	 *
	 * @param TrackIndex The index of the track.
	 * @param FormatIndex Index of the track format.
	 * @return Video dimensions.
	 * @see GetVideoTrackAspectRatio, GetVideoTrackFrameRate, GetVideoTrackFrameRates, GetVideoTrackType
	 */
	MEDIAUTILS_API FIntPoint GetVideoTrackDimensions(int32 TrackIndex, int32 FormatIndex) const;

	/**
	 * Get frame rate of the specified video track.
	 *
	 * @param TrackIndex The index of the track.
	 * @param FormatIndex Index of the track format.
	 * @return Video frame rate.
	 * @see GetVideoTrackAspectRatio, GetVideoTrackDimensions, GetVideoTrackFrameRates, GetVideoTrackType
	 */
	MEDIAUTILS_API float GetVideoTrackFrameRate(int32 TrackIndex, int32 FormatIndex) const;

	/**
	 * Get the supported range of frame rates of the specified video track.
	 *
	 * @param TrackIndex The index of the track.
	 * @param FormatIndex Index of the track format.
	 * @return Frame rate range (in frames per second).
	 * @see GetVideoTrackAspectRatio, GetVideoTrackDimensions, GetVideoTrackFrameRate, GetVideoTrackType
	 */
	MEDIAUTILS_API TRange<float> GetVideoTrackFrameRates(int32 TrackIndex, int32 FormatIndex) const;

	/**
	 * Get the type of the specified video track format.
	 *
	 * @param TrackIndex The index of the track.
	 * @param FormatIndex Index of the track format.
	 * @return Video format type string.
	 * @see GetVideoTrackAspectRatio, GetVideoTrackDimensions, GetVideoTrackFrameRate, GetVideoTrackFrameRates
	 */
	MEDIAUTILS_API FString GetVideoTrackType(int32 TrackIndex, int32 FormatIndex) const;

	/**
	 * Get the field of view.
	 *
	 * @param OutHorizontal Will contain the horizontal field of view.
	 * @param OutVertical Will contain the vertical field of view.
	 * @return true on success, false if feature is not available or if field of view has never been set.
	 * @see GetViewOrientation, SetViewField
	 */
	MEDIAUTILS_API bool GetViewField(float& OutHorizontal, float& OutVertical) const;

	/**
	 * Get the view's orientation.
	 *
	 * @param OutOrientation Will contain the view orientation.
	 * @return true on success, false if feature is not available or if orientation has never been set.
	 * @see GetViewField, SetViewOrientation
	 */
	MEDIAUTILS_API bool GetViewOrientation(FQuat& OutOrientation) const;

	/**
	 * Check whether the player is in an error state.
	 *
	 * @see IsReady
	 */
	MEDIAUTILS_API bool HasError() const;

	/**
	 * Whether the player is currently buffering data.
	 *
	 * @return true if buffering, false otherwise.
	 * @see GetState, IsConnecting, IsLooping
	 */
	MEDIAUTILS_API bool IsBuffering() const;

	/**
	 * Whether the player is currently connecting to a media source.
	 *
	 * @return true if connecting, false otherwise.
	 * @see GetState, IsBuffering, IsLooping
	 */
	MEDIAUTILS_API bool IsConnecting() const;

	/**
	 * Whether playback is looping.
	 *
	 * @return true if looping, false otherwise.
	 * @see GetState, IsBuffering, IsConnecting, SetLooping
	 */
	MEDIAUTILS_API bool IsLooping() const;

	/**
	 * Whether playback is currently paused.
	 *
	 * @return true if playback is paused, false otherwise.
	 * @see CanPause, IsPlaying, IsReady, Pause
	 */
	MEDIAUTILS_API bool IsPaused() const;

	/**
	 * Whether playback is in progress.
	 *
	 * @return true if playback has started, false otherwise.
	 * @see CanPlay, IsPaused, IsReady, Play
	 */
	MEDIAUTILS_API bool IsPlaying() const;

	/**
	 * Whether the media is currently opening or buffering.
	 *
	 * @return true if playback is being prepared, false otherwise.
	 * @see CanPlay, IsPaused, IsReady, Play
	 */
	MEDIAUTILS_API bool IsPreparing() const;

	/**
	 * Whether media is currently closed.
	 *
	 * @return true if media is closed, false otherwise.
	 */
	MEDIAUTILS_API bool IsClosed() const;

	/**
	 * Whether media is ready for playback.
	 *
	 * A player is ready for playback if it has a media source opened that
	 * finished preparing and is not in an error state.
	 *
	 * @return true if media is ready, false otherwise.
	 * @see HasError, IsPaused, IsPlaying, Stop
	 */
	MEDIAUTILS_API bool IsReady() const;

	/**
	 * Open a media source from a URL with optional parameters.
	 *
	 * @param Url The URL of the media to open (file name or web address).
	 * @param Options Optional media parameters.
	 * @param PlayerOptions Optional player parameters.
	 * @return true if the media is being opened, false otherwise.
	 */
	MEDIAUTILS_API bool Open(const FString& Url, const IMediaOptions* Options, const FMediaPlayerOptions* PlayerOptions = nullptr);

	/**
	 * Query the time ranges of cached media samples for the specified caching state.
	 *
	 * @param State The sample state we're interested in.
	 * @param OutTimeRanges Will contain the set of matching sample time ranges.
	 */
	MEDIAUTILS_API void QueryCacheState(EMediaTrackType TrackType, EMediaCacheState State, TRangeSet<FTimespan>& OutTimeRanges) const;

	/**
	 * Seeks to the specified playback time.
	 *
	 * @param Time The playback time to set.
	 * @return true on success, false otherwise.
	 * @see GetTime, Rewind
	 */
	MEDIAUTILS_API bool Seek(const FTimespan& Time);

	/**
	 * A seek has been requested, waiting to receive samples.
	 * @see Seek
	 */
	MEDIAUTILS_API bool IsSeeking() const;

	/**
	 * If valid, returns the current seek request.
	 * @see Seek, IsSeeking
	 */
	MEDIAUTILS_API FMediaTimeStamp GetSeekTarget() const;

	/**
	 *	Set seek to be done after current seek is completed.
	 *	@remark This value is reset if Seek is called directly.
	 */
	MEDIAUTILS_API void SetNextSeek(const FTimespan& InTime);

	/**
	 * Select the active track of the given type.
	 *
	 * The selected track will use its currently active format. Active formats will
	 * be remembered on a per track basis. The first available format is active by
	 * default. To switch the track format, use SetTrackFormat instead.
	 *
	 * @param TrackType The type of track to select.
	 * @param TrackIndex The index of the track to select, or INDEX_NONE to deselect.
	 * @return true if the track was selected, false otherwise.
	 * @see GetNumTracks, GetSelectedTrack, SetTrackFormat
	 */
	MEDIAUTILS_API bool SelectTrack(EMediaTrackType TrackType, int32 TrackIndex);

	/**
	 * Set the time on which to block.
	 *
	 * If set, this player will block in TickFetch until the video sample
	 * for the specified time are actually available.
	 *
	 * @param Time The time to block on, or FTimespan::MinValue to disable.
	 * @see TickFetch
	 * @note Deprecated: Use SetBlockOnTimeRange instead
	 */
	MEDIAUTILS_API void SetBlockOnTime(const FTimespan& Time);

	/**
	 * Set the time range on which to block.
	 *
	 * If set, this player will block in TickFetch until the video sample
	 * for the specified time are actually available.
	 *
	 * @param TimeRange The time range to block on, use empty range to disable
	 */
	MEDIAUTILS_API void SetBlockOnTimeRange(const TRange<FTimespan>& TimeRange);

	/**
	 * Set sample caching options.
	 *
	 * @param Ahead Duration of samples to cache ahead of the play head.
	 * @param Behind Duration of samples to cache behind the play head.
	 */
	MEDIAUTILS_API void SetCacheWindow(FTimespan Ahead, FTimespan Behind);

	/**
	 * Set the player's globally unique identifier.
	 *
	 * @param Guid The GUID to set.
	 * @see GetGuid
	 */
	MEDIAUTILS_API void SetGuid(FGuid& Guid);

	/**
	 * Enables or disables playback looping.
	 *
	 * @param Looping Whether playback should be looped.
	 * @return true on success, false otherwise.
	 * @see IsLooping
	 */
	MEDIAUTILS_API bool SetLooping(bool Looping);

	/**
	 * Changes media ooptions on the player.
	 */
	MEDIAUTILS_API void SetMediaOptions(const IMediaOptions* Options);

	/**
	 * Changes the media's playback rate.
	 *
	 * @param Rate The playback rate to set.
	 * @return true on success, false otherwise.
	 * @see GetRate, SupportsRate
	 */
	MEDIAUTILS_API bool SetRate(float Rate);

	/**
	 * Sets a new media playback range.
	 * Has an effect only if SupportsPlaybackTimeRange() returns true and the media supports it.
	 * A live stream cannot be constrained to a range.
	 * The range will be clamped if necessary to be within the media's absolute time range.
	 * Changing the time range may trigger an implicit Seek() depending on where the current
	 * playback position is located with regard to the new range.
	 * Unless prevented by the media a playback range can be cleared by passing an empty range.
	 *
	 * @param InTimeRange The new playback range to set.
	 * @return true if successful, false otherwise.
	 * @see SupportsPlaybackTimeRange, GetPlaybackTimeRange
	 */
	MEDIAUTILS_API bool SetPlaybackTimeRange(const TRange<FTimespan>& InTimeRange);

	/**
	 * Changes the media's native volume.
	 *
	 * @param Rate The volume to set.
	 * @return true on success, false otherwise.
	 * @see NativeAudioOut
	 */
	MEDIAUTILS_API bool SetNativeVolume(float Volume);

	/**
	 * Set the format on the specified track.
	 *
	 * Selecting the format will not switch to the specified track. To switch
	 * tracks, use SelectTrack instead. If the track is already selected, the
	 * format change will be applied immediately.
	 *
	 * @param TrackType The type of track to update.
	 * @param TrackIndex The index of the track to update.
	 * @param FormatIndex The index of the format to select (must be valid).
	 * @return true if the track was selected, false otherwise.
	 * @see GetNumTrackFormats, GetNumTracks, GetTrackFormat, SelectTrack
	 */
	MEDIAUTILS_API bool SetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex, int32 FormatIndex);

	/**
	 * Set the frame rate of the specified video track.
	 *
	 * @param TrackIndex The index of the track, or INDEX_NONE for the selected one.
	 * @param FormatIndex Index of the track format, or INDEX_NONE for the selected one.
	 * @param FrameRate The frame rate to set (must be in range of format's supported frame rates).
	 * @return true on success, false otherwise.
	 * @see GetVideoTrackAspectRatio, GetVideoTrackDimensions, GetVideoTrackFrameRate, GetVideoTrackFrameRates, GetVideoTrackType
	 */
	MEDIAUTILS_API bool SetVideoTrackFrameRate(int32 TrackIndex, int32 FormatIndex, float FrameRate);

	/**
	 * Set the field of view.
	 *
	 * @param Horizontal Horizontal field of view (in Euler degrees).
	 * @param Vertical Vertical field of view (in Euler degrees).
	 * @param Whether the field of view change should be absolute (true) or relative (false).
	 * @return true on success, false otherwise.
	 * @see GetViewField, SetViewOrientation
	 */
	MEDIAUTILS_API bool SetViewField(float Horizontal, float Vertical, bool Absolute);

	/**
	 * Set the view's orientation.
	 *
	 * @param Orientation Quaternion representing the orientation.
	 * @param Whether the orientation change should be absolute (true) or relative (false).
	 * @return true on success, false otherwise.
	 * @see GetViewOrientation, SetViewField
	 */
	MEDIAUTILS_API bool SetViewOrientation(const FQuat& Orientation, bool Absolute);

	/**
	 * Whether the specified playback rate is supported.
	 *
	 * @param Rate The playback rate to check.
	 * @param Unthinned Whether no frames should be dropped at the given rate.
	 * @see CanScrub, CanSeek
	 */
	MEDIAUTILS_API bool SupportsRate(float Rate, bool Unthinned) const;

	/**
	 * Record last audio sample played to track audio sync (for automated tests)
	 *
	 * @param SampleTime Time of media sample currently being played
	 * @return true if playback is being prepared, false otherwise.
	 */
	MEDIAUTILS_API void SetLastAudioRenderedSampleTime(FTimespan SampleTime);

	/**
	 * Get time of last audio sample played
	 *
	 * @return Time of last audio sample played.
	 */
	MEDIAUTILS_API FTimespan GetLastAudioRenderedSampleTime() const;

	/**
	 * Sets whether the player can broadcast events when running on a thread other than the game thread.
	 *
	 * @param bInAreEventsSafeForAnyThread If true then allow broadcast when not on the game thread.
	 */
	MEDIAUTILS_API void SetAreEventsSafeForAnyThread(bool bInAreEventsSafeForAnyThread);

public:

	/** Get an event delegate that is invoked when a media event occurred. */
	DECLARE_EVENT_OneParam(FMediaPlayerFacade, FOnMediaEvent, EMediaEvent /*Event*/)
	FOnMediaEvent& OnMediaEvent()
	{
		return MediaEvent;
	}

public:

	//~ IMediaClockSink interface

	MEDIAUTILS_API virtual void TickFetch(FTimespan DeltaTime, FTimespan Timecode) override;
	MEDIAUTILS_API virtual void TickInput(FTimespan DeltaTime, FTimespan Timecode) override;
	MEDIAUTILS_API virtual void TickOutput(FTimespan DeltaTime, FTimespan Timecode) override;

public:

	//~ IMediaTickable interface

	MEDIAUTILS_API virtual void TickTickable() override;

protected:

	/**
	 * Whether sample fetching should block.
	 *
	 * @return true if sample fetching should block, false otherwise.
	 */
	MEDIAUTILS_API bool BlockOnFetch() const;

	/** Flush all media sample sinks & player plugin. */
	MEDIAUTILS_API void Flush(bool bExcludePlayer = false, bool bOnSeek = false);

	/** Internal function to retrieve the current timestamp */
	MEDIAUTILS_API FMediaTimeStamp GetTimeStampInternal(bool bForDisplay) const;

	/**
	 * Get details about the specified audio track format.
	 *
	 * @param TrackIndex The index of the audio track.
	 * @param FormatIndex The index of the track's format.
	 * @param OutFormat Will contain the format details.
	 * @return true on success, false otherwise.
	 * @see GetVideoTrackFormat
	 */
	MEDIAUTILS_API bool GetAudioTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaAudioTrackFormat& OutFormat) const;

	/**
	 * Get a player that can play the specified media URL.
	 *
	 * @param Url The URL to play.
	 * @param Options The media options for the URL.
	 * @return The player if found, or nullptr otherwise.
	 */
	MEDIAUTILS_API IMediaPlayerFactory *GetPlayerFactoryForUrl(const FString& Url, const IMediaOptions* Options) const;

	/**
	 * Get details about the specified audio track format.
	 *
	 * @param TrackIndex The index of the audio track.
	 * @param FormatIndex The index of the track's format.
	 * @param OutFormat Will contain the format details.
	 * @return true on success, false otherwise.
	 * @see GetVideoTrackFormat
	 */
	MEDIAUTILS_API bool GetVideoTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaVideoTrackFormat& OutFormat) const;

	/**
	 * Process the given media event.
	 *
	 * @param Event The event to process.
	 * @param bIsBroadcastAllowed If true then we can broadcast events, if false then they will sent when possible.
	 **/
	MEDIAUTILS_API void ProcessEvent(EMediaEvent Event, bool bIsBroadcastAllowed);

private:
	struct FTrackSelection
	{
		int32 UserSelection[(int32)EMediaTrackType::Num];
		int32 PlayerSelection[(int32)EMediaTrackType::Num];
	} TrackSelection;

	/** Reset all tracks. */
	MEDIAUTILS_API void ResetTracks();

	/** Setup track selection with player. */
	MEDIAUTILS_API void UpdateTrackSelectionWithPlayer();

	/** Select the default media tracks. */
	MEDIAUTILS_API void SelectDefaultTracks();

protected:
	MEDIAUTILS_API bool HaveAudioPlayback() const;
	MEDIAUTILS_API bool HaveVideoPlayback() const;
	MEDIAUTILS_API float GetUnpausedRate() const;


protected:

	//~ IMediaEventSink interface

	MEDIAUTILS_API void ReceiveMediaEvent(EMediaEvent Event) override;

private:
	friend class FMediaPlayerLifecycleManagerDelegateControl;

	// Internal
	MEDIAUTILS_API bool NotifyLifetimeManagerDelegate_PlayerOpen(IMediaPlayerLifecycleManagerDelegate::IControlRef& NewLifecycleManagerDelegateControl, const FString& InUrl, const IMediaOptions* Options, const FMediaPlayerOptions* InPlayerOptions, IMediaPlayerFactory* PlayerFactory, bool bWillCreatePlayer, uint32 WillUseNewResources, uint64 NewPlayerInstanceID);
	MEDIAUTILS_API bool NotifyLifetimeManagerDelegate_PlayerCreated();
	MEDIAUTILS_API bool NotifyLifetimeManagerDelegate_PlayerCreateFailed();
	MEDIAUTILS_API bool NotifyLifetimeManagerDelegate_PlayerClosed();
	MEDIAUTILS_API bool NotifyLifetimeManagerDelegate_PlayerDestroyed();
	MEDIAUTILS_API bool NotifyLifetimeManagerDelegate_PlayerResourcesReleased(uint32 ResourceFlags);

	MEDIAUTILS_API void ProcessAudioSamples(IMediaSamples& Samples, const TRange<FMediaTimeStamp>& TimeRange);
	MEDIAUTILS_API bool ProcessVideoSamples(IMediaSamples& Samples, const TRange<FMediaTimeStamp>& TimeRange);
	MEDIAUTILS_API void ProcessCaptionSamples(IMediaSamples& Samples, const TRange<FMediaTimeStamp>& TimeRange);
	MEDIAUTILS_API void ProcessSubtitleSamples(IMediaSamples& Samples, const TRange<FMediaTimeStamp>& TimeRange);
	MEDIAUTILS_API void ProcessMetadataSamples(IMediaSamples& Samples, const TRange<FMediaTimeStamp>& TimeRange);

	MEDIAUTILS_API void ProcessAudioSamplesV1(IMediaSamples& Samples, TRange<FTimespan> TimeRange);
	MEDIAUTILS_API void ProcessVideoSamplesV1(IMediaSamples& Samples, TRange<FTimespan> TimeRange);
	MEDIAUTILS_API void ProcessSubtitleSamplesV1(IMediaSamples& Samples, TRange<FTimespan> TimeRange);
	MEDIAUTILS_API void ProcessCaptionSamplesV1(IMediaSamples& Samples, TRange<FTimespan> TimeRange);
	MEDIAUTILS_API void ProcessMetadataSamplesV1(IMediaSamples& Samples, TRange<FTimespan> TimeRange);

	MEDIAUTILS_API bool IsVideoSampleStillGood(const TRange<FMediaTimeStamp>& LastSampleTimeRange, const TRange<FMediaTimeStamp>& TimeRange, bool bReverse) const;
	MEDIAUTILS_API void MonitorAudioEnablement();
	MEDIAUTILS_API void UpdateSeekStatus(const FMediaTimeStamp* pCheckTimeStamp = nullptr);
	void ExecuteNextSeek();
	void PrepareSampleQueueForSequenceIndex();
	MEDIAUTILS_API void PreSampleProcessingTimeHandling();
	MEDIAUTILS_API bool GetCurrentPlaybackTimeRange(TRange<FMediaTimeStamp>& TimeRange, float Rate, FTimespan DeltaTime, bool bPurgeSampleRelated) const;
	MEDIAUTILS_API void PostSampleProcessingTimeHandling(FTimespan DeltaTime);

	MEDIAUTILS_API void DestroyPlayer();

	MEDIAUTILS_API bool ContinueOpen(IMediaPlayerLifecycleManagerDelegate::IControlRef NewLifecycleManagerDelegateControl, const FString& Url, const IMediaOptions* Options, const FMediaPlayerOptions* PlayerOptions, IMediaPlayerFactory* PlayerFactory, TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> ReusedPlayer, bool bCreateNewPlayer, uint64 NewPlayerInstanceID);

	MEDIAUTILS_API void SendSinkEvent(EMediaSampleSinkEvent Event, const FMediaSampleSinkEventData& Data);

	/** Audio sample sinks. */
	TWeakPtr<FMediaAudioSampleSink, ESPMode::ThreadSafe> PrimaryAudioSink;
	FMediaAudioSampleSinks AudioSampleSinks;

	/** Caption sample sinks. */
	FMediaOverlaySampleSinks CaptionSampleSinks;

	/** Metadata sample sinks. */
	FMediaBinarySampleSinks MetadataSampleSinks;

	/** Subtitle sample sinks. */
	FMediaOverlaySampleSinks SubtitleSampleSinks;

	/** Video sample sinks. */
	FMediaVideoSampleSinks VideoSampleSinks;

private:
	MEDIAUTILS_API TRange<FMediaTimeStamp> GetAdjustedBlockOnRange() const;

	TRange<FTimespan> GetActivePlaybackRange() const;
	void ReInit();

	class FBlockOnRange
	{
	public:
		FBlockOnRange(FMediaPlayerFacade* InFacade) : Facade(InFacade) { Reset(); }

		void SetRange(const TRange<FTimespan> & NewRange);

		const TRange<FMediaTimeStamp> & GetRange() const;
		bool IsSet() const;

		void OnFlush();
		void OnSeek(int32 PrimaryIndex);

		void Reset()
		{
			BlockOnRange = TRange<FMediaTimeStamp>::Empty();
			CurrentTimeRange = TRange<FTimespan>::Empty();
			LastProcessedTimeRange = TRange<FTimespan>::Empty();
			RangeIsDirty = false;
			OnBlockSequenceIndex = 0;
			OnBlockLoopIndexOffset = 0;
		}

	private:
		/** The hosting player facade */
		FMediaPlayerFacade* Facade;

		/** The most recently set time range to block on */
		TRange<FTimespan> CurrentTimeRange;

		/** Last user provided BlockOnRange value */
		mutable TRange<FTimespan> LastProcessedTimeRange;

		/** The time range to block on sample fetching. */
		mutable TRange<FMediaTimeStamp> BlockOnRange;

		/** Flag to indicate if internal range is valid or not */
		mutable bool RangeIsDirty;

		/** Primary ("seek") sequence index used during blocked playback processing */
		mutable int32 OnBlockSequenceIndex;

		/** Secondary (loop) sequence index offset used during blocked playback processing */
		mutable int32 OnBlockLoopIndexOffset;
	};

	FBlockOnRange BlockOnRange;


	/** Flag to indicate block on range feature as disabled for the current playback session due to previous timeout */
	bool BlockOnRangeDisabled;

	/** Media sample cache. */
	FMediaSampleCache* Cache;

	/** Synchronizes access to Player. */
	FCriticalSection CriticalSection;

	/** The URL of the currently loaded media source. */
	FString CurrentUrl;

	/** The last used non-zero play rate (zero if playback never started). */
	float LastRate;

	/** The last rate set with the facade (unfiltered by player). */
	float CurrentRate;

	/** Flag indicating that we have an active audio setup */
	bool bHaveActiveAudio;

	/** Flag indicating the current availability of media samples. **/
	int32 VideoSampleAvailability;
	int32 AudioSampleAvailability;

	/** An event delegate that is invoked when a media event occurred. */
	FOnMediaEvent MediaEvent;

	/** Time of the next expected video sample (used for block on fetch). */
	FTimespan NextVideoSampleTime;

	/** The low-level player used to play the media source. */
	TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> Player;

	/** Low level player instance ID */
	uint64 PlayerInstanceID = ~0;

	/** Low level player will use callback to notify of resource release */
	bool PlayerUsesResourceReleaseNotification = false;

	/** Media player Guid */
	FGuid PlayerGuid;

	/** Media player event queue. */
	TQueue<EMediaEvent, EQueueMode::Mpsc> QueuedEvents;

	/** Queue to hold events that we could not broadcast yet. */
	TQueue<EMediaEvent, EQueueMode::Spsc> QueuedEventBroadcasts;

	/** CS to make last time values below thread safe */
	mutable FCriticalSection TimeValueLock;

	/** Time of last audio sample played. */
	FMediaTimeStampSample LastAudioRenderedSampleTime;

	/** Time of last audio sample decoded. */
	FMediaTimeStampSample LastAudioSampleProcessedTime;

	/** Time/Range of last video sample decoded. */
	TRange<FMediaTimeStamp> LastVideoSampleProcessedTimeRange;

	/** Timestamp for audio considered "current" for this frame (use ONLY for return to outside code) */
	FMediaTimeStamp CurrentFrameAudioTimeStamp;

	/** Timestamp for video considered "current" for this frame (stays valid even after a flush, until new data comes in) */
	FMediaTimeStamp CurrentFrameVideoTimeStamp;

	/** Timestamp for video considered "current" for this frame for GUI / display purposes (stays valid even after a flush, until new data comes in) */
	FMediaTimeStamp CurrentFrameVideoDisplayTimeStamp;

	/** Estimation for next frame's video timestamp (used when no audio present or active in stream) */
	FMediaTimeStampSample NextEstVideoTimeAtFrameStart;

	/** Timecode of the video frame most recently delivered to the video sinks */
	TOptional<FTimecode> MostRecentlyDeliveredVideoFrameTimecode;

	/** Timestamp of seek target location if seek is pending */
	FMediaTimeStamp SeekTargetTime;

	/** When multiple consecutive seek are issued while seeking, we keep track of the most recent request. */
	TOptional<FTimespan> NextSeekTime;

	/** The next sequence index value we expect to see on the samples provided by a v2 timing enabled player. */
	TOptional<int32> NextSequenceIndex;

	/** Latch for error state of the most recently used player to be queried after it may have been closed. **/
	bool bDidRecentPlayerHaveError;

	/** If true, then we can send out events even if we are not on the game thread. */
	bool bAreEventsSafeForAnyThread;

	/** Mediamodule we are working in */
	IMediaModule* MediaModule;

	/** UMediaplayer we are working with */
	TWeakObjectPtr<UMediaPlayer> MediaPlayer;

	/** Control interface for lifecycle manager delegate */
	IMediaPlayerLifecycleManagerDelegate::IControlRef LifecycleManagerDelegateControl;
};
