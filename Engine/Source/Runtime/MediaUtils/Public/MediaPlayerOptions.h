// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "Containers/Map.h"
#include "Misc/Variant.h"

#include "MediaPlayerOptions.generated.h"

UENUM(BlueprintType)
enum class EMediaPlayerOptionBooleanOverride : uint8
{
	UseMediaPlayerSetting,
	Enabled,
	Disabled
};


UENUM(BlueprintType)
enum class EMediaPlayerOptionSeekTimeType : uint8
{
	// Ignore the given value and lets the media player choose.
	Ignored,
	// Given seek time is relative to the start of the media.
	RelativeToStartTime
};


UENUM(BlueprintType)
enum class EMediaPlayerOptionTrackSelectMode : uint8
{
	// Let the media player choose defaults.
	UseMediaPlayerDefaults UMETA(DisplayName="Media player selects default tracks"),
	// Use fixed track indices as specified with MediaPlayerTrackOptions
	UseTrackOptionIndices UMETA(DisplayName="Uses provided track indices"),
	// Use language codes as specified with MediaPlayerInitialTrackLanguageSelection
	UseLanguageCodes UMETA(DisplayName="Use provided language codes or have media player select defaults when there is no match")
};


USTRUCT(BlueprintType)
struct FMediaPlayerTrackOptions
{
	GENERATED_BODY()

	FMediaPlayerTrackOptions() :
		Audio(0),
		Caption(-1),
		Metadata(-1),
		Script(-1),
		Subtitle(-1),
		Text(-1),
		Video(0)
	{
	}

	/** Check if this Media Player Track Options is not equal to another one. */
	bool operator!=(const FMediaPlayerTrackOptions& Other) const
	{
		return (Audio != Other.Audio
			|| Caption != Other.Caption
			|| Metadata != Other.Metadata
			|| Script != Other.Script
			|| Subtitle != Other.Subtitle
			|| Text != Other.Text
			|| Video != Other.Video);
	}

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Tracks")
	int32 Audio;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Tracks")
	int32 Caption;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Tracks")
	int32 Metadata;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Tracks")
	int32 Script;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Tracks")
	int32 Subtitle;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Tracks")
	int32 Text;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Tracks")
	int32 Video;
};


USTRUCT(BlueprintType)
struct FMediaPlayerInitialTrackLanguageSelection
{
	GENERATED_BODY()

	FMediaPlayerInitialTrackLanguageSelection()
	{ }

	bool operator!=(const FMediaPlayerInitialTrackLanguageSelection& Other) const
	{ return Video != Other.Video || Audio != Other.Audio || Subtitle != Other.Subtitle || Caption != Other.Caption; }

	// Language code or codes of the video track to select. Useful when the video contains burned-in subtitles or localized scenes.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Tracks", meta=(ToolTip="Comma separated RFC 4647 language ranges in order of priority, or keep empty for default."))
	FString Video;

	// Language code or codes of the audio track to select.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Tracks", meta=(ToolTip="Comma separated RFC 4647 language ranges in order of priority, or keep empty for default."))
	FString Audio;

	// Language code or codes of the subtitle track to select.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Tracks", meta=(ToolTip="Comma separated RFC 4647 language ranges in order of priority, or keep empty for default."))
	FString Subtitle;

	// Language code or codes of the caption track to select. Captions may not provide language codes if carried in the video signal.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Tracks", meta=(ToolTip="Comma separated RFC 4647 language ranges in order of priority, or keep empty for default."))
	FString Caption;
};

USTRUCT(BlueprintType)
struct FMediaPlayerOptions
{
	GENERATED_BODY()

	FMediaPlayerOptions() :
		TrackSelection(EMediaPlayerOptionTrackSelectMode::UseTrackOptionIndices),
		SeekTime(0),
		SeekTimeType(EMediaPlayerOptionSeekTimeType::RelativeToStartTime),
		PlayOnOpen(EMediaPlayerOptionBooleanOverride::UseMediaPlayerSetting),
		Loop(EMediaPlayerOptionBooleanOverride::UseMediaPlayerSetting)
	{
	}

	void SetAllAsOptional()
	{
		SeekTime = FTimespan::MinValue();
		TrackSelection = EMediaPlayerOptionTrackSelectMode::UseMediaPlayerDefaults;
		SeekTimeType = EMediaPlayerOptionSeekTimeType::Ignored;
		PlayOnOpen = EMediaPlayerOptionBooleanOverride::UseMediaPlayerSetting;
		Loop = EMediaPlayerOptionBooleanOverride::UseMediaPlayerSetting;
	}

	/** Check if this Media Options is not equal to another one. */
	bool operator!=(const FMediaPlayerOptions& Other) const
	{
		return (Tracks != Other.Tracks
			|| TracksByLanguage != Other.TracksByLanguage
			|| TrackSelection != Other.TrackSelection
			|| SeekTime != Other.SeekTime
			|| SeekTimeType != Other.SeekTimeType
			|| PlayOnOpen != Other.PlayOnOpen
			|| Loop != Other.Loop);
	}

	/** Fixed indices of media tracks to select. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Tracks", meta=(DisplayName="Initial track indices", ToolTip="Indices of the media tracks to select for playback", EditCondition="TrackSelection==EMediaPlayerOptionTrackSelectMode::UseTrackOptionIndices"))
	FMediaPlayerTrackOptions Tracks;

	/** Track selection by language. Not supported by all media players. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Tracks", meta=(DisplayName="Initial track language", ToolTip="Language of the media tracks to select for playback", EditCondition="TrackSelection==EMediaPlayerOptionTrackSelectMode::UseLanguageCodes"))
	FMediaPlayerInitialTrackLanguageSelection TracksByLanguage;

	/** How to select the initial media tracks. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Tracks", meta=(DisplayName="Initial track selection mode", ToolTip="How the initial media tracks for playback are selected"))
	EMediaPlayerOptionTrackSelectMode TrackSelection;

	/** Initial media time to start playback at. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Misc", meta=(EditCondition="SeekTimeType!=EMediaPlayerOptionSeekTimeType::Ignored"))
	FTimespan SeekTime;

	/** How to interpret the initial seek time. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Misc", meta=(DisplayName="Seek time interpretation"))
	EMediaPlayerOptionSeekTimeType SeekTimeType;

	/** How to handle automatic playback when media opens. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Misc")
	EMediaPlayerOptionBooleanOverride PlayOnOpen;

	/** How to initially select looping of the media. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Misc")
	EMediaPlayerOptionBooleanOverride Loop;


	/** Custom options used internally. Must not be serialized or editable via blueprint. */
	TMap<FName, FVariant> InternalCustomOptions;
};

namespace MediaPlayerOptionValues
{
	inline const FName& Environment()
	{ static FName OptName("Environment"); return OptName; }
	inline const FName& Environment_Sequencer()
	{ static FName OptName("Sequencer"); return OptName; }
	inline const FName& Environment_Preview()
	{ static FName OptName("Preview"); return OptName; }

	inline const FName& ImgMediaSmartCacheEnabled()
	{ static FName OptName("ImgMediaSmartCacheEnabled"); return OptName; }
	inline const FName& ImgMediaSmartCacheTimeToLookAhead()
	{ static FName OptName("ImgMediaSmartCacheTimeToLookAhead"); return OptName; }

	/**
	 * Option to specify a view media texture (as a soft object path string)
	 * to allow players supporting mips and tiles to preload only the required set for the given texture.
	 */
	inline const FName& ViewMediaTexture()
	{ static FName OptName("ViewMediaTexture"); return OptName; }

	/**
	 * Requests parsing timecode meta data if available.
	 */
	inline const FName& ParseTimecodeInfo()
	{ static FName OptName("ParseTimecodeInfo"); return OptName; }
}
