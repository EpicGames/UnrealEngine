// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "IImageWrapper.h"
#include "MediaSampleQueue.h"
#include "Misc/Timespan.h"
#include "Templates/SharedPointer.h"

class FMediaPlayerFacade;
class FMediaRecorderClockSink;
class IImageWriteQueue;


/**
 * Records samples from a media player.
 * Loop, seek or reverse and not supported.
 * Currently only records texture samples and that support sample format 8bit BGRA, half float RBGA
 */
class FMediaRecorder
{
public:

	/** Default constructor. */
	MEDIAUTILS_API FMediaRecorder();

public:

	enum class EMediaRecorderNumerationStyle
	{
		AppendFrameNumber,
		AppendSampleTime
	};

	struct FMediaRecorderData
	{
		/** The media player facade to record from. */
		TSharedRef<FMediaPlayerFacade, ESPMode::ThreadSafe> PlayerFacade;
		/** BaseFilename(including with path and filename) for each recorded frames. */
		FString BaseFilename;
		/** How to numerate the filename. */
		EMediaRecorderNumerationStyle NumerationStyle;
		/** The format to save the image to. */
		EImageFormat TargetImageFormat;
		/** If the format support it, set the alpha to 1 (or 255) */
		bool bResetAlpha;
		/**
		 * An image format specific compression setting.
		 * For EXRs, either 0 (Default) or 1 (Uncompressed)
		 * For others, a value between 1 (worst quality, best compression) and 100 (best quality, worst compression)
		 */
		int32 CompressionQuality;

		FMediaRecorderData(const TSharedRef<FMediaPlayerFacade, ESPMode::ThreadSafe>& InPlayerFacade, const FString& InBaseFilename)
			: PlayerFacade(InPlayerFacade)
			, BaseFilename(InBaseFilename)
			, NumerationStyle(EMediaRecorderNumerationStyle::AppendSampleTime)
			, TargetImageFormat(EImageFormat::EXR)
			, bResetAlpha(false)
			, CompressionQuality(0)
		{
		}
	};

public: 

	/**
	 * Start recording samples from a given media player.
	 */
	MEDIAUTILS_API void StartRecording(const FMediaRecorderData& InRecoderData);

	/**
	 * Stop recording media samples.
	 */
	MEDIAUTILS_API void StopRecording();

	/**
	 * Is currently recording media samples.
	 */
	bool IsRecording() const { return bRecording; }

	/**
	 * Blocking call that will wait for all frames to be recorded before returning.
	 */
	MEDIAUTILS_API bool WaitPendingTasks(const FTimespan& InDuration);

protected:

	/**
	 * Tick the recorder.
	 *
	 * @param Timecode The current timecode.
	 */
	MEDIAUTILS_API void TickRecording();

private:

	friend class FMediaRecorderClockSink;

	/** The recorder's media clock sink. */
	TSharedPtr<FMediaRecorderClockSink, ESPMode::ThreadSafe> ClockSink;

	/** Texture sample queue. */
	TSharedPtr<FMediaTextureSampleQueue, ESPMode::ThreadSafe> SampleQueue;

	/** Whether recording is in progress. */
	bool bRecording;

	/** Warning for unsupported format have been showed. */
	bool bUnsupportedWarningShowed;

	/** Number of frames recorded. */
	int32 FrameCount;

	/** Media Recorder Data saved options. */
	FString BaseFilename;
	EMediaRecorderNumerationStyle NumerationStyle;
	EImageFormat TargetImageFormat;
	bool bSetAlpha;
	int32 CompressionQuality;

	/** The image writer. */
	IImageWriteQueue* ImageWriteQueue;
	TFuture<void> CompletedFence;
};
