// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/LruCache.h"
#include "Containers/Set.h"
#include "CoreTypes.h"
#include "HAL/CriticalSection.h"
#include "IMediaAudioSample.h"
#include "IMediaBinarySample.h"
#include "IMediaOverlaySample.h"
#include "IMediaTextureSample.h"
#include "Math/RangeSet.h"
#include "Misc/Timespan.h"
#include "Templates/SharedPointer.h"

class IMediaAudioSample;
class IMediaBinarySample;
class IMediaOverlaySample;
class IMediaPlayer;
class IMediaSamples;
class IMediaTextureSample;
template <typename ElementType> class TRangeSet;


/**
 * Implements a cache for media samples.
 */
class FMediaSampleCache
{
	/** Key functions template for the sample cache sets. */
	template<typename SampleType>
	struct TSampleKeyFuncs
		: BaseKeyFuncs<TSharedPtr<SampleType, ESPMode::ThreadSafe>, FTimespan, false>
	{
		static inline FTimespan GetSetKey(const TSharedPtr<SampleType, ESPMode::ThreadSafe>& Element)
		{
			return Element->GetTime().Time;
		}

		static inline bool Matches(FTimespan A, FTimespan B)
		{
			return A == B;
		}

		static inline uint32 GetKeyHash(FTimespan Key)
		{
			return GetTypeHash(Key);
		}
	};

	/** Template for sample cache sets. */
	template<typename SampleType>
	using TSampleSet = TSet<TSharedPtr<SampleType, ESPMode::ThreadSafe>, TSampleKeyFuncs<SampleType>>;

public:

	/** Default constructor. */
	MEDIAUTILS_API FMediaSampleCache();

public:

	/**
	 * Empty the cache.
	 *
	 * @see Update
	 */
	MEDIAUTILS_API void Empty();

	/**
	 * Get the audio sample for the specified play time.
	 *
	 * This method returns the sample that contains the specified time code.
	 *
	 * @param Time The time to get the sample for (in the player's clock).
	 * @return The sample, or nullptr if no sample available.
	 * @see GetAudioSamples, GetCachedVideoSampleRanges, GetOverlaySamples
	 */
	MEDIAUTILS_API TSharedPtr<IMediaAudioSample, ESPMode::ThreadSafe> GetAudioSample(FTimespan Time);

	/**
	 * Get the time ranges of audio samples currently in the cache.
	 *
	 * @param OutTimeRanges Will contain the set of cached sample time ranges.
	 * @see GetAudioSample, GetCachedVideoSampleRanges
	 */
	MEDIAUTILS_API void GetCachedAudioSampleRanges(TRangeSet<FTimespan>& OutTimeRanges) const;

	/**
	 * Get the time ranges of video samples currently in the cache.
	 *
	 * @param OutTimeRanges Will contain the set of cached sample time ranges.
	 * @see GetCachedAudioSampleRanges, GetVideoSample
	 */
	MEDIAUTILS_API void GetCachedVideoSampleRanges(TRangeSet<FTimespan>& OutTimeRanges) const;

	/**
	 * Get the text overlay samples for the specified time.
	 *
	 * @param Time The time to get the sample for (in the player's clock).
	 * @param OutSamples Will contain the overlay samples.
	 * @see GetAudioSample, GetVideoSample
	 */
	MEDIAUTILS_API void GetOverlaySamples(FTimespan Time, TArray<TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe>>& OutSamples);

	/**
	 * Get the video sample for the specified play time.
	 *
	 * This method will return the closest match for the given time code. If the current play
	 * rate is positive, the closest sample with an equal or older time is returned. If the
	 * current rate is negative, the closest sample with an equal or newer time is returned.
	 *
	 * @param Time The time to get the sample for (in the player's clock).
	 * @param Forward Whether the play direction is forward (true) or backward (false).
	 * @return The sample, or nullptr if no sample available.
	 * @see GetAudioSample, GetCachedVideoSampleRanges, GetOverlaySamples
	 */
	MEDIAUTILS_API TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe> GetVideoSample(FTimespan Time, bool Forward);

	/**
	 * Set the time window of samples to cache.
	 *
	 * @param Ahead Maximum time of samples to cache ahead of the play position.
	 * @param Behind Maximum time of samples to cache behind the play position.
	 */
	void SetCacheWindow(FTimespan Ahead, FTimespan Behind)
	{
		CacheAhead = Ahead;
		CacheBehind = Behind;
	}

	/**
	 * Tick the cache.
	 *
	 * This method fetches any unconsumed media samples from the player
	 * and removes expired samples from the cache.
	 *
	 * @param DeltaTime Time since the last tick.
	 * @param Rate The current play rate.
	 * @param Time The current play time.
	 * @see Initialize, Shutdown
	 */
	MEDIAUTILS_API void Tick(FTimespan DeltaTime, float Rate, FTimespan Time);

private:

	/** Cached audio samples. */
	TSampleSet<IMediaAudioSample> AudioSamples;

	/** Cached metadata samples. */
	TSampleSet<IMediaBinarySample> MetadataSamples;

	/** Cached overlay samples. */
	TSampleSet<IMediaOverlaySample> OverlaySamples;

	/** Cached video samples. */
	TSampleSet<IMediaTextureSample> VideoSamples;

private:

	FTimespan CacheAhead;

	FTimespan CacheBehind;

	/** Synchronizes access to sample collections. */
	mutable FCriticalSection CriticalSection;
};
