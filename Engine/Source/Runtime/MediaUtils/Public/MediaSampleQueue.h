// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Queue.h"
#include "HAL/PlatformAtomics.h"
#include "IMediaSamples.h"
#include "HAL/CriticalSection.h"
#include "Math/Interval.h"
#include "Misc/App.h"
#include "Misc/ScopeLock.h"
#include "Misc/Timespan.h"
#include "Templates/SharedPointer.h"

#include "MediaSampleSink.h"
#include "MediaSampleSource.h"

#include "IMediaTimeSource.h"
#include "IMediaAudioSample.h"
#include "IMediaTextureSample.h"
#include "IMediaBinarySample.h"
#include "IMediaOverlaySample.h"


enum class EMediaSampleQueueFetchResult
{
	Found,
	None,
	PurgedToEmpty
};

/**
 * Template for media sample queues.
 */
template<typename SampleType, typename SinkType=TMediaSampleSink<SampleType>>
class TMediaSampleQueue
	: public SinkType
	, public TMediaSampleSource<SampleType>
{
public:

	TMediaSampleQueue(int32 InMaxSamplesInQueue = -1)
		: MaxSamplesInQueue(InMaxSamplesInQueue)
	{ }

	/** Virtual destructor. */
	virtual ~TMediaSampleQueue() = default;

public:

	/**
	 * Get the number of samples in the queue.
	 *
	 * Note: The value returned by this function is only eventually consistent. It
	 * can be called by both consumer and producer threads, but it should not be used
	 * to query the actual state of the queue. Always use Dequeue and Peek instead!
	 *
	 * @return Number of samples.
	 * @see Enqueue, Dequeue, Peek
	 */
	int32 Num() const
	{
		return Samples.Num();
	}

	/**
	 * Returns the number of samples that were dropped in any of the member functions.
	 * This is only for tracking statistics and may not necessarily be accurate.
	 * The count is never implicitly cleared by any member function. To clear
	 * it call with bInClearToZero set to true.
	 *
	 * @return Number of samples that were dropped.
	 */
	uint32 GetNumDroppedSamples(bool bInClearToZero)
	{
		uint32 n = NumDroppedSamples;
		if (bInClearToZero)
		{
			NumDroppedSamples = 0;
		}
		return n;
	}

public:

	//~ TMediaSampleSource interface (to be called only from consumer thread)

	virtual bool Dequeue(TSharedPtr<SampleType, ESPMode::ThreadSafe>& OutSample) override
	{
		FScopeLock Lock(&CriticalSection);

		if (Samples.Num() == 0)
		{
			return false; // empty queue
		}

		TSharedPtr<SampleType, ESPMode::ThreadSafe> Sample(Samples[0]);

		if (!Sample.IsValid())
		{
			return false; // pending flush
		}

		Samples.RemoveAt(0);

		OutSample = Sample;

		return true;
	}

	virtual bool Peek(TSharedPtr<SampleType, ESPMode::ThreadSafe>& OutSample) override
	{
		FScopeLock Lock(&CriticalSection);

		if (Samples.Num() == 0)
		{
			return false; // empty queue
		}

		TSharedPtr<SampleType, ESPMode::ThreadSafe> Sample(Samples[0]);

		if (!Sample.IsValid())
		{
			return false; // pending flush
		}

		OutSample = Sample;

		return true;
	}

	virtual void GetSampleTimes(TArray<TRange<FMediaTimeStamp>>& OutSampleTimeRanges) override
	{
		FScopeLock Lock(&CriticalSection);
		for(int32 i=0,iMax=Samples.Num(); i<iMax; ++i)
		{
			if (Samples[i].IsValid())
			{
				OutSampleTimeRanges.Emplace(TRange<FMediaTimeStamp>(Samples[i]->GetTime(), Samples[i]->GetTime() + Samples[i]->GetDuration()));
			}
		}
	}

	virtual bool Pop() override
	{
		FScopeLock Lock(&CriticalSection);

		if (Samples.Num() == 0)
		{
			return false; // empty queue
		}

		if (!Samples[0].IsValid())
		{
			return false; // pending flush
		}

		Samples.RemoveAt(0);

		return true;
	}

	bool Discard(const TRange<FMediaTimeStamp>& TimeRange, bool bReverse)
	{
		// Code below assumes a fully specified range, no open bounds!
		check(TimeRange.HasLowerBound() && TimeRange.HasUpperBound());

		FScopeLock Lock(&CriticalSection);

		int32 FirstPossibleIndex, LastPossibleIndex, NumOldSamplesAtBegin;
		FindRangeInQueue(TimeRange, bReverse, FirstPossibleIndex, LastPossibleIndex, NumOldSamplesAtBegin);

		// Found anything?
		if (FirstPossibleIndex >= 0)
		{
			check(LastPossibleIndex >= 0);

			// Remove all samples indicated...
			const int32 NumToRemove = LastPossibleIndex - FirstPossibleIndex + 1;
			Samples.RemoveAt(FirstPossibleIndex, NumToRemove);
			NumDroppedSamples += (uint32)NumToRemove;
			return true;
		}
		return false;
	}

	EMediaSampleQueueFetchResult FetchBestSampleForTimeRange(const TRange<FMediaTimeStamp>& TimeRange, TSharedPtr<SampleType, ESPMode::ThreadSafe>& OutSample, bool bReverse, bool bConsistentResult)
	{
		// Notes:
		// - Reverse playback still works with increasing indices in the queue. PTS values will be going down in it, rather than up,
		//   but the order of indices is still identical.
		// - The code below must be able to deal with time ranges that span loop points (different secondary sequence indices)

		// Code below assumes a fully specified range, no open bounds!
		check(TimeRange.HasLowerBound() && TimeRange.HasUpperBound());

		OutSample.Reset();

		FScopeLock Lock(&CriticalSection);

		int32 FirstPossibleIndex, LastPossibleIndex, NumOldSamplesAtBegin;
		FindRangeInQueue(TimeRange, bReverse, FirstPossibleIndex, LastPossibleIndex, NumOldSamplesAtBegin);

		// Found anything?
		if (FirstPossibleIndex >= 0)
		{
			if (!bConsistentResult)
			{
				//
				// Return the latest sample with the most "coverage" for the given time range that can be fetched
				// (this naturally depends on how many samples or available and hence timing - the result is not consistent between instances or repeat runs)
				//
				if (FirstPossibleIndex != LastPossibleIndex)
				{
					// More then one sample. Find the one that fits the bill, best...
					// (we look for the one with the largest overlap & newest time)
					FMediaTimeStamp BestDuration(FTimespan::Zero(), -1);
					int32 BestIndex = FirstPossibleIndex;
					for (int32 Idx = FirstPossibleIndex; Idx <= LastPossibleIndex; ++Idx)
					{
						const TSharedPtr<SampleType, ESPMode::ThreadSafe>& Sample = Samples[Idx];

						// Check once more if this sample is actually overlapping as we may get non-monotonically increasing data...
						TRange<FMediaTimeStamp> SampleTimeRange = TRange<FMediaTimeStamp>(Sample->GetTime(), Sample->GetTime() + Sample->GetDuration());

						if (TimeRange.Overlaps(SampleTimeRange))
						{
							// Ok. This one is real, see if it is a better fit than the last one...
							TRange<FMediaTimeStamp> SampleInRangeRange(TRange<FMediaTimeStamp>::Intersection(SampleTimeRange, TimeRange));

							FMediaTimeStamp SampleDuration(SampleInRangeRange.Size<FMediaTimeStamp>());
							if (SampleDuration >= BestDuration)
							{
								BestDuration = SampleDuration;
								BestIndex = Idx;
							}
						}
					}

					check(BestIndex >= NumOldSamplesAtBegin);

					// Found the best. Return it & delete all candidate samples up and including it from the queue
					OutSample = Samples[BestIndex];
					int32 NumToRemove = BestIndex - FirstPossibleIndex + 1;
					Samples.RemoveAt(FirstPossibleIndex, NumToRemove);
					NumDroppedSamples += NumToRemove > 1 ? (uint32)(NumToRemove - 1) : 0;
				}
				else
				{
					// Single sample found: we just take it!
					OutSample = Samples[FirstPossibleIndex];
					Samples.RemoveAt(FirstPossibleIndex);
				}
			}
			else
			{
				//
				// Return the first sample with maximum possible coverage given the time range or nothing if the sample is not yet in the queue
				// (this yields reproducible results between instances as far as the selection of frames is concerned if the passed in ranges are identical in each run / instance)
				//

				for (int32 Idx = FirstPossibleIndex; Idx <= LastPossibleIndex; ++Idx)
				{
					const TSharedPtr<SampleType, ESPMode::ThreadSafe>& Sample = Samples[Idx];

					TRange<FMediaTimeStamp> SampleTimeRange = TRange<FMediaTimeStamp>(Sample->GetTime(), Sample->GetTime() + Sample->GetDuration());

					check(TimeRange.Overlaps(SampleTimeRange));
					TRange<FMediaTimeStamp> SampleInRangeRange(TRange<FMediaTimeStamp>::Intersection(SampleTimeRange, TimeRange));

					FMediaTimeStamp SampleDuration(SampleInRangeRange.Size<FMediaTimeStamp>());

					// Do we either have full coverage or no "better" sample could be after this one and still in range?
					if (!bReverse ? (SampleDuration.Time >= Sample->GetDuration() || (TimeRange.GetUpperBoundValue() - (Sample->GetTime() + Sample->GetDuration())) < SampleDuration)
								  : (SampleDuration.Time >= Sample->GetDuration() || (Sample->GetTime() - TimeRange.GetLowerBoundValue()) < SampleDuration))
					{
						// Yes, so we return this one and remove all candidates and this one from the queue
						OutSample = Sample;
						int32 NumToRemove = Idx - FirstPossibleIndex + 1;
						Samples.RemoveAt(FirstPossibleIndex, NumToRemove);
						NumDroppedSamples += NumToRemove > 1 ? (uint32)(NumToRemove - 1) : 0;
						break;
					}
				}
			}
		}

		// Any frames considered outdated?
		if (NumOldSamplesAtBegin != 0)
		{
			// Cleanup samples that are now considered outdated...
			Samples.RemoveAt(0, NumOldSamplesAtBegin);
			NumDroppedSamples += (uint32)NumOldSamplesAtBegin;
		}

		// Return true if we got a sample...
		return OutSample.IsValid() ? EMediaSampleQueueFetchResult::Found : (NumOldSamplesAtBegin && Samples.IsEmpty() ? EMediaSampleQueueFetchResult::PurgedToEmpty : EMediaSampleQueueFetchResult::None);
	}


	uint32 PurgeOutdatedSamples(const FMediaTimeStamp& ReferenceTime, bool bReversed, FTimespan MaxAge)
	{
		FScopeLock Lock(&CriticalSection);

		int32 Num = Samples.Num();
		if (Num > 0)
		{
			// All samples at or beyond the reference time are good to stay
			int32 Idx;
			if (!bReversed)
			{
				for (Idx = Num - 1; Idx >= 0; --Idx)
				{
					if (Samples[Idx]->GetTime() < ReferenceTime)
					{
						break;
					}
				}
			}
			else
			{
				for (Idx = Num - 1; Idx >= 0; --Idx)
				{
					if (Samples[Idx]->GetTime() > ReferenceTime)
					{
						break;
					}
				}
			}
			// Accumulate durations of samples from the reference time backwards to judge what to purge as "too old"
			FTimespan Age = FTimespan::Zero();
			for (; Idx >= 0; --Idx)
			{
				auto Duration = Samples[Idx]->GetDuration();
				Age += Duration;
				if (Age > MaxAge)
				{
					// All earlier samples, including the current one are "too old"
					Samples.RemoveAt(0, Idx + 1);
					NumDroppedSamples += (uint32)(Idx + 1);
					return Idx + 1;
				}
			}
		}
		return 0;
	}

	void PurgeUntilSequenceIndex(int32 InUntilIndex)
	{
		FScopeLock Lock(&CriticalSection);
		while(Samples.Num() && Samples[0]->GetTime().GetSequenceIndex() < InUntilIndex)
		{
			Samples.RemoveAt(0);
			++NumDroppedSamples;
		}
	}

public:

	//~ TMediaSampleSink interface (to be called only from producer thread)

	virtual bool Enqueue(const TSharedRef<SampleType, ESPMode::ThreadSafe>& Sample) override
	{
		FScopeLock Lock(&CriticalSection);

		if ((MaxSamplesInQueue > 0) && (Samples.Num() >= MaxSamplesInQueue))
		{
			return false;
		}

		Samples.Push(Sample);
		return true;
	}

	virtual void RequestFlush() override
	{
		FScopeLock Lock(&CriticalSection);
		Samples.Empty();
		++FlushCount;
	}

	virtual uint32 GetFlushCount() const
	{
		FScopeLock Lock(&CriticalSection);
		return FlushCount;
	}

	virtual bool CanAcceptSamples(int32 NumSamples) const override
	{
		return (MaxSamplesInQueue < 0) || ((Samples.Num() + NumSamples) <= MaxSamplesInQueue);
	}

protected:
	void FindRangeInQueue(const TRange<FMediaTimeStamp>& TimeRange, bool bReverse, int32& FirstPossibleIndex, int32& LastPossibleIndex, int32& NumOldSamplesAtBegin)
	{
		FirstPossibleIndex = -1;
		LastPossibleIndex = -1;
		NumOldSamplesAtBegin = 0;

		int32 Num = Samples.Num();
		if (Num > 0)
		{
			for (int32 Idx = 0; Idx < Num; ++Idx)
			{
				const TSharedPtr<SampleType, ESPMode::ThreadSafe>& Sample = Samples[Idx];
				TRange<FMediaTimeStamp> SampleTimeRange = TRange<FMediaTimeStamp>(Sample->GetTime(), Sample->GetTime() + Sample->GetDuration());

				if (TimeRange.Overlaps(SampleTimeRange))
				{
					// Sample is at least partially inside the requested range, recall the range of samples we find...
					if (FirstPossibleIndex < 0)
					{
						FirstPossibleIndex = Idx;
					}
					LastPossibleIndex = Idx;
				}
				else
				{
					if (!bReverse ? (SampleTimeRange.GetLowerBoundValue() >= TimeRange.GetUpperBoundValue()) :
									(SampleTimeRange.GetUpperBoundValue() <= TimeRange.GetLowerBoundValue()))
					{
						// Sample is entirely past requested time range, we can stop
						// (we assume monotonically increasing time stamps here)
						break;
					}

					// If the incoming data it not monotonically increasing we might get here after we already found the first overlapping sample
					// -> we do not count further non-overlapping, older samples into this range
					if (FirstPossibleIndex < 0)
					{
						// Sample is before time range, we will delete is later, no reason to keep it
						++NumOldSamplesAtBegin;
					}
					else
					{
						// If we find an older non-overlapping sample after an overlapping one, we move the last possible index on to ensure these samples die ASAP
						LastPossibleIndex = Idx;
					}
				}
			}
		}

	}

	mutable FCriticalSection CriticalSection;
	TArray<TSharedPtr<SampleType, ESPMode::ThreadSafe>> Samples;
	int32 MaxSamplesInQueue;
	uint32 FlushCount = 0;
	uint32 NumDroppedSamples = 0;
};


/** audio sample queue. */
class FMediaAudioSampleQueue : public TMediaSampleQueue<class IMediaAudioSample, class FMediaAudioSampleSink>
{
public:
	FMediaAudioSampleQueue(uint32 MaxSamplesInQueue = -1)
		: TMediaSampleQueue<class IMediaAudioSample, class FMediaAudioSampleSink>(MaxSamplesInQueue)
	{ }

	void SetAudioTime(const FMediaTimeStampSample& InAudioTime)
	{
		FScopeLock Lock(&CriticalSection);
		AudioTime = InAudioTime;
	}

	void SetAudioTimeIfEqualFlushCount(const FMediaTimeStampSample& InAudioTime, uint32 InFlushCount)
	{
		FScopeLock Lock(&CriticalSection);
		if (InFlushCount == FlushCount)
		{
			AudioTime = InAudioTime;
		}
	}

	FMediaTimeStampSample GetAudioTime() const override
	{
		FScopeLock Lock(&CriticalSection);
		return AudioTime;
	}

	void InvalidateAudioTime() override
	{
		FScopeLock Lock(&CriticalSection);
		AudioTime.Invalidate();
	}

	virtual void RequestFlush() override
	{
		FScopeLock Lock(&CriticalSection);
		TMediaSampleQueue<class IMediaAudioSample, class FMediaAudioSampleSink>::RequestFlush();
		AudioTime.Invalidate();
	}
private:
	FMediaTimeStampSample AudioTime;
};

/** Type definition for binary sample queue. */
typedef TMediaSampleQueue<class IMediaBinarySample> FMediaBinarySampleQueue;

/** Type definition for overlay sample queue. */
typedef TMediaSampleQueue<class IMediaOverlaySample> FMediaOverlaySampleQueue;

/** Type definition for texture sample queue. */
typedef TMediaSampleQueue<class IMediaTextureSample> FMediaTextureSampleQueue;
