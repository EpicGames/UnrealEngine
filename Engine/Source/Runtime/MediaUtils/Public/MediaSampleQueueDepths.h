// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/** Legacy player output queue depth maximums
 * @note These are not used inside MediaFramework and are meant to support older player sources code only.
*/
struct FMediaPlayerQueueDepths
{
	static const int32 MaxAudioSinkDepth = 512;
	static const int32 MaxCaptionSinkDepth = 256;
	static const int32 MaxMetadataSinkDepth = 256;
	static const int32 MaxSubtitleSinkDepth = 256;
	static const int32 MaxVideoSinkDepth = 8;
};
