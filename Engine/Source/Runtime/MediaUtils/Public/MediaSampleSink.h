// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Timespan.h"
#include "Templates/SharedPointer.h"
#include "Delegates/Delegate.h"
#include "IMediaTimeSource.h"
#include "IMediaTracks.h"

class UMediaPlayer;

/** Events send from player to sink */
enum class EMediaSampleSinkEvent
{
	Attached,				//!< Attached to a UMediaPlayer
	Detached,				//!< Detached from a UMediaPlayer
	PlayerPluginChange,		//!< Player plugin used changed
	SampleDataUpdate,		//!< Sample data was updated
	FlushWasRequested,		//!< Flush has been requested
	MediaClosed,			//!< Media has been closed
	PlaybackEndReached,		//!< End of playback has been reached
	PlaybackRateChanged,	//!< Rate of playback changed
};

struct FMediaSampleSinkEventData
{
	union {
		struct
		{
			UMediaPlayer* MediaPlayer;
		} Attached;
		struct
		{
			UMediaPlayer* MediaPlayer;
		} Detached;
		struct
		{
			UMediaPlayer* MediaPlayer;
		} PlayerPluginChange;
		struct
		{
			uint32 Dummy;
		} SampleDataUpdate;
		struct
		{
			UMediaPlayer* MediaPlayer;
		} FlushWasRequested;
		struct
		{
			uint32 Dummy;
		} MediaClosed;
		struct
		{
			uint32 Dummy;
		} PlaybackEndReached;
		struct
		{
			float PlaybackRate;
		} PlaybackRateChanged;
	};
};

/**
 * Interface for media sample sinks.
 *
 * This interface declares the write side of media sample queues.
 *
 * @see TMediaSampleQueue
 */
template<typename SampleType>
class TMediaSampleSink
{
public:

	/**
	 * Add a sample to the head of the queue.
	 *
	 * @param Sample The sample to add.
	 * @return true if the item was added, false otherwise.
	 * @see Num, RequestFlush
	 */
	virtual bool Enqueue(const TSharedRef<SampleType, ESPMode::ThreadSafe>& Sample) = 0;

	/**
	 * Get the number of samples in the queue.
	 *
	 * @return Number of samples.
	 * @see Enqueue, RequestFlush
	 */
	virtual int32 Num() const = 0;

	/**
	 * Check if sink can accept new samples
	 *
	 * @param NumSamples How many samples we would like the sink to accept
	 * @return True if samples could be accepted, false otherwise
	 * @note Override in implementation as needed
	 */
	virtual bool CanAcceptSamples(int32 NumSamples) const
	{
		return true;
	}

	/**
	 * Request to flush the queue.
	 *
	 * @note To be called only from producer thread.
	 * @see Enqueue, Num
	 */
	virtual void RequestFlush() = 0;

	/**
	 * Returns the number of times the queue has been requested to flush already.
	 * @return Number of times RequestFlush() was called.
	 */
	virtual uint32 GetFlushCount() const = 0;

	/**
	 * Receive event
	 */
	void ReceiveEvent(EMediaSampleSinkEvent Event, const FMediaSampleSinkEventData& Data)
	{
		MediaSampleSinkEvent.Broadcast(Event, Data);
	}

	/**
	* Register to receive events flowing to this sink
	*/
	DECLARE_EVENT_TwoParams(TMediaSampleSink<SampleType>, FOnMediaSampleSinkEvent, EMediaSampleSinkEvent /*Event*/, const FMediaSampleSinkEventData& /*Data*/)
	FOnMediaSampleSinkEvent& OnMediaSampleSinkEvent()
	{
		return MediaSampleSinkEvent;
	}

public:

	/** Virtual destructor. */
	virtual ~TMediaSampleSink() = default;

private:
	FOnMediaSampleSinkEvent MediaSampleSinkEvent;
};


/** Type definition for audio sample sink. */
class FMediaAudioSampleSink : public TMediaSampleSink<class IMediaAudioSample>
{
public:
	/**
	 * Get last sampled current audio timestamp being played
	 */
	virtual FMediaTimeStampSample GetAudioTime() const = 0;
	virtual void InvalidateAudioTime() = 0;
};

/** Type definition for binary sample sink. */
typedef TMediaSampleSink<class IMediaBinarySample> FMediaBinarySampleSink;

/** Type definition for overlay sample sink. */
typedef TMediaSampleSink<class IMediaOverlaySample> FMediaOverlaySampleSink;

/** Type definition for texture sample sink. */
typedef TMediaSampleSink<class IMediaTextureSample> FMediaTextureSampleSink;
