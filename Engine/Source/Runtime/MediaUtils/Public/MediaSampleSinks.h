// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Templates/SharedPointer.h"

#include "MediaSampleSink.h"


/**
 * Collection of media sample sinks.
 *
 * @param SampleType The type of media samples that the sinks process.
 */
template<typename SampleType, typename SinkType=TMediaSampleSink<SampleType>>
class TMediaSampleSinks
{
public:

	/**
	 * Add the given media sample sink to the collection.
	 *
	 * @param SampleSink The sink to add.
	 * @see Num, Remove
	 */
	void Add(const TSharedRef<SinkType, ESPMode::ThreadSafe>& SampleSink, UMediaPlayer* Player = nullptr)
	{
		if (Sinks.Find(SampleSink) != INDEX_NONE)
		{
			return;
		}
		Sinks.Add(SampleSink);
		FMediaSampleSinkEventData Data;
		Data.Attached = { Player };
		SampleSink->ReceiveEvent(EMediaSampleSinkEvent::Attached, Data);
	}

	/**
	 * Enqueue the given media samples to the registered sinks.
	 *
	 * This method will also remove expired sinks that haven't been removed yet.
	 *
	 * @param Sample The media sample to enqueue.
	 * @param MaxQueueDepth The maximum depth of the sink queues before overflow.
	 * @return true if the sample was enqueued to all sinks, false if one or more sinks overflowed.
	 * @see Flush
	 */
	bool Enqueue(const TSharedRef<SampleType, ESPMode::ThreadSafe>& Sample)
	{
		bool Overflowed = false;

		for (int32 SinkIndex = Sinks.Num() - 1; SinkIndex >= 0; --SinkIndex)
		{
			TSharedPtr<SinkType, ESPMode::ThreadSafe> Sink = Sinks[SinkIndex].Pin();

			if (Sink.IsValid())
			{
				if (!Sink->CanAcceptSamples(1))
				{
					Overflowed = true;
				}
				else
				{
					Sink->Enqueue(Sample);
					FMediaSampleSinkEventData Data;
					Sink->ReceiveEvent(EMediaSampleSinkEvent::SampleDataUpdate, Data);
				}
			}
			else
			{
				Sinks.RemoveAt(SinkIndex);
			}
		}

		return !Overflowed;
	}

	/**
	 * Flush all registered sinks.
	 *
	 * This method will also remove expired sinks that haven't been removed yet.
	 *
	 * @see Enqueue
	 */
	void Flush(UMediaPlayer* MediaPlayer)
	{
		for (int32 SinkIndex = Sinks.Num() - 1; SinkIndex >= 0; --SinkIndex)
		{
			TSharedPtr<SinkType, ESPMode::ThreadSafe> Sink = Sinks[SinkIndex].Pin();

			if (Sink.IsValid())
			{
				Sink->RequestFlush();

				FMediaSampleSinkEventData Data;
				Data.FlushWasRequested = { MediaPlayer };
				Sink->ReceiveEvent(EMediaSampleSinkEvent::FlushWasRequested, Data);
			}
			else
			{
				Sinks.RemoveAt(SinkIndex);
			}
		}
	}

	/**
	 * Get the number of sinks in this collection.
	 *
	 * @return Number of sinks.
	 * @see Add, Remove
	 */
	int32 Num() const
	{
		return Sinks.Num();
	}

	/**
	* Check if any sinks are inserted
	* 
	* @return True if empty, false otherwise
	*/
	bool IsEmpty() const
	{
		return Sinks.IsEmpty();
	}

	/**
	 * Remove the given media sample sink from the collection.
	 *
	 * @param SampleSink The sink to remove.
	 * @see Add, Num
	 */
	void Remove(const TSharedRef<SinkType, ESPMode::ThreadSafe>& SampleSink, UMediaPlayer* Player = nullptr)
	{
		if (Sinks.RemoveSingle(SampleSink))
		{
			FMediaSampleSinkEventData Data;
			Data.Detached = { Player };
			SampleSink->ReceiveEvent(EMediaSampleSinkEvent::Detached, Data);
		}
	}

	/**
	 * Remove any invalid sinks
	 */
	void Cleanup()
	{
		for (int32 SinkIndex = Sinks.Num() - 1; SinkIndex >= 0; --SinkIndex)
		{
			if (!Sinks[SinkIndex].IsValid())
			{
				Sinks.RemoveAt(SinkIndex);
			}
		}
	}

	/**
	 * Receive event and broadcast to sinks
	 */
	void ReceiveEvent(EMediaSampleSinkEvent Event, const FMediaSampleSinkEventData& Data)
	{
		for (int32 SinkIndex = Sinks.Num() - 1; SinkIndex >= 0; --SinkIndex)
		{
			TSharedPtr<SinkType, ESPMode::ThreadSafe> Sink = Sinks[SinkIndex].Pin();

			if (Sink.IsValid())
			{
				Sink->ReceiveEvent(Event, Data);
			}
			else
			{
				Sinks.RemoveAt(SinkIndex);
			}
		}
	}

protected:

	/** The collection of registered sinks. */
	TArray<TWeakPtr<SinkType, ESPMode::ThreadSafe>> Sinks;
};

typedef TMediaSampleSinks<IMediaTextureSample, FMediaTextureSampleSink> FMediaVideoSampleSinks;

class FMediaAudioSampleSinks : public TMediaSampleSinks<IMediaAudioSample, FMediaAudioSampleSink>
{
public:
	/**
	 * Get primary audio sink and cleanup any invalid sinks
	 */
	TSharedPtr<FMediaAudioSampleSink, ESPMode::ThreadSafe> GetPrimaryAudioSink()
	{
		Cleanup();
		if (Sinks.Num() == 0)
		{
			return TSharedPtr<FMediaAudioSampleSink, ESPMode::ThreadSafe>();
		}
		return Sinks[0].Pin();
	}
};

typedef TMediaSampleSinks<IMediaOverlaySample, FMediaOverlaySampleSink> FMediaOverlaySampleSinks;
typedef TMediaSampleSinks<IMediaBinarySample, FMediaBinarySampleSink> FMediaBinarySampleSinks;
