// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "IMediaTimeSource.h"
#include "Templates/SharedPointer.h"
#include "Containers/Array.h"
#include "Math/Range.h"

/**
 * Interface for media sample sources.
 *
 * This interface declares the read side of media sample queues.
 *
 * @see TMediaSampleQueue
 */
template<typename SampleType>
class TMediaSampleSource
{
public:

	/**
	 * Remove and return the next sample in the queue.
	 *
	 * @param OutSample Will contain the sample if the queue is not empty.
	 * @return true if a sample has been returned, false if the queue was empty.
	 * @see Peek, Pop
	 */
	virtual bool Dequeue(TSharedPtr<SampleType, ESPMode::ThreadSafe>& OutSample) = 0;

	/**
	 * Peek at the next sample in the queue without removing it.
	 *
	 * @param OutSample Will contain the sample if the queue is not empty.
	 * @return true if a sample has been returned, false if the queue was empty.
	 * @see Dequeue, Pop
	 */
	virtual bool Peek(TSharedPtr<SampleType, ESPMode::ThreadSafe>& OutSample) = 0;

	/**
	 * Returns the sample start and end times of all samples currently in the queue.
	 * @param OutSampleTimeRanges Will contain the sample time ranges.
	 */
	virtual void GetSampleTimes(TArray<TRange<FMediaTimeStamp>>& OutSampleTimeRanges) = 0;

	/**
	 * Remove the next sample from the queue.
	 *
	 * @return true if a sample was removed, false otherwise.
	 * @see Dequeue, Peek
	 */
	virtual bool Pop() = 0;

public:

	/** Virtual destructor. */
	virtual ~TMediaSampleSource() = default;
};


/** Type definition for audio sample source. */
typedef TMediaSampleSource<class IMediaAudioSample> FMediaAudioSampleSource;

/** Type definition for binary sample source. */
typedef TMediaSampleSource<class IMediaBinarySample> FMediaBinarySampleSource;

/** Type definition for overlay sample source. */
typedef TMediaSampleSource<class IMediaOverlaySample> FMediaOverlaySampleSource;

/** Type definition for texture sample source. */
typedef TMediaSampleSource<class IMediaTextureSample> FMediaTextureSampleSource;
