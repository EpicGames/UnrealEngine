// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "IMediaSamples.h"
#include "IMediaTextureSample.h"
#include "IMediaTimeSource.h"
#include "MediaSampleQueue.h"
#include "Templates/SharedPointer.h"

class IMediaAudioSample;
class IMediaBinarySample;
class IMediaOverlaySample;
class IMediaTextureSample;
struct FTimespan;
template <typename ElementType> class TRange;


/**
 * General purpose media sample queue.
 */
class FMediaSamples
	: public IMediaSamples
{
public:
	static constexpr uint32 MaxNumberOfQueuedVideoSamples = 4;
	static constexpr uint32 MaxNumberOfQueuedAudioSamples = 4;
	static constexpr uint32 MaxNumberOfQueuedCaptionSamples = 4;
	static constexpr uint32 MaxNumberOfQueuedSubtitlesSamples = 4;
	static constexpr uint32 MaxNumberOfQueuedMetaDataSamples = 4;

	MEDIAUTILS_API FMediaSamples(uint32 InMaxNumberOfQueuedAudioSamples = MaxNumberOfQueuedAudioSamples,
				  uint32 InMaxNumberOfQueuedVideoSamples = MaxNumberOfQueuedVideoSamples,
				  uint32 InMaxNumberOfQueuedCaptionSamples = MaxNumberOfQueuedCaptionSamples,
				  uint32 InMaxNumberOfQueuedSubtitlesSamples = MaxNumberOfQueuedSubtitlesSamples,
				  uint32 InMaxNumberOfQueuedMetaDataSamples = MaxNumberOfQueuedMetaDataSamples);

	MEDIAUTILS_API ~FMediaSamples();

	/**
	 * Add the given audio sample to the cache.
	 *
	 * @param Sample The sample to add.
	 * @see AddCaption, AddMetadata, AddSubtitle, AddVideo, NumAudio
	 */
	MEDIAUTILS_API void AddAudio(const TSharedRef<IMediaAudioSample, ESPMode::ThreadSafe>& Sample);

	/**
	 * Add the given caption sample to the cache.
	 *
	 * @param Sample The sample to add.
	 * @see AddAudio, AddMetadata, AddSubtitle, AddVideo, NumCaption
	 */
	MEDIAUTILS_API void AddCaption(const TSharedRef<IMediaOverlaySample, ESPMode::ThreadSafe>& Sample);

	/**
	 * Add the given audio sample to the cache.
	 *
	 * @param Sample The sample to add.
	 * @see AddAudio, AddCaption, AddSubtitle, AddVideo, NumMetadata
	 */
	MEDIAUTILS_API void AddMetadata(const TSharedRef<IMediaBinarySample, ESPMode::ThreadSafe>& Sample);

	/**
	 * Add the given subtitle sample to the cache.
	 *
	 * @param Sample The sample to add.
	 * @see AddAudio, AddCaption, AddMetadata, AddVideo, NumSubtitle
	 */
	MEDIAUTILS_API void AddSubtitle(const TSharedRef<IMediaOverlaySample, ESPMode::ThreadSafe>& Sample);

	/**
	 * Add the given audio sample to the cache.
	 *
	 * @param Sample The sample to add.
	 * @see AddAudio, AddCaption, AddMetadata, AddSubtitle, NumVideo
	 */
	MEDIAUTILS_API void AddVideo(const TSharedRef<IMediaTextureSample, ESPMode::ThreadSafe>& Sample);
	/**
	 * Get the number of queued audio samples.
	 *
	 * @return Number of samples.
	 * @see AddAudio, NumCaption, NumMetadata, NumSubtitle, NumVideo
	 */
	int32 NumAudioSamples() const override
	{
		return AudioSampleQueue.Num();
	}

	/**
	 * Get the number of queued caption samples.
	 *
	 * @return Number of samples.
	 * @see AddCaption, NumAudio, NumMetadata, NumSubtitle, NumVideo
	 */
	int32 NumCaptionSamples() const override
	{
		return CaptionSampleQueue.Num();
	}

	/**
	 * Get the number of queued metadata samples.
	 *
	 * @return Number of samples.
	 * @see AddMetadata, NumAudio, NumCaption, NumSubtitle, NumVideo
	 */
	int32 NumMetadataSamples() const override
	{
		return MetadataSampleQueue.Num();
	}

	/**
	 * Get the number of queued subtitle samples.
	 *
	 * @return Number of samples.
	 * @see AddSubtitle, NumAudio, NumCaption, NumMetadata, NumVideo
	 */
	int32 NumSubtitleSamples() const override
	{
		return SubtitleSampleQueue.Num();
	}

	/**
	 * Get the number of queued video samples.
	 *
	 * @return Number of samples.
	 * @see AddVideo, NumAudio, NumCaption, NumMetadata, NumSubtitle
	 */
	int32 NumVideoSamples() const override
	{
		return VideoSampleQueue.Num();
	}

public:

	//~ IMediaSamples interface

	MEDIAUTILS_API virtual bool FetchAudio(TRange<FTimespan> TimeRange, TSharedPtr<IMediaAudioSample, ESPMode::ThreadSafe>& OutSample) override;
	MEDIAUTILS_API virtual bool FetchCaption(TRange<FTimespan> TimeRange, TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe>& OutSample) override;
	MEDIAUTILS_API virtual bool FetchMetadata(TRange<FTimespan> TimeRange, TSharedPtr<IMediaBinarySample, ESPMode::ThreadSafe>& OutSample) override;
	MEDIAUTILS_API virtual bool FetchSubtitle(TRange<FTimespan> TimeRange, TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe>& OutSample) override;
	MEDIAUTILS_API virtual bool FetchVideo(TRange<FTimespan> TimeRange, TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& OutSample) override;
	MEDIAUTILS_API virtual void FlushSamples() override;

	MEDIAUTILS_API virtual bool FetchAudio(TRange<FMediaTimeStamp> TimeRange, TSharedPtr<IMediaAudioSample, ESPMode::ThreadSafe>& OutSample) override;
	MEDIAUTILS_API virtual bool FetchCaption(TRange<FMediaTimeStamp> TimeRange, TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe>& OutSample) override;
	MEDIAUTILS_API virtual bool FetchMetadata(TRange<FMediaTimeStamp> TimeRange, TSharedPtr<IMediaBinarySample, ESPMode::ThreadSafe>& OutSample) override;
	MEDIAUTILS_API virtual bool FetchSubtitle(TRange<FMediaTimeStamp> TimeRange, TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe>& OutSample) override;
	MEDIAUTILS_API virtual bool FetchVideo(TRange<FMediaTimeStamp> TimeRange, TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& OutSample) override;

	MEDIAUTILS_API virtual EFetchBestSampleResult FetchBestVideoSampleForTimeRange(const TRange<FMediaTimeStamp>& TimeRange, TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& OutSample, bool bReverse, bool bConsistentResult) override;

	MEDIAUTILS_API virtual void SetMinExpectedNextSequenceIndex(TOptional<int32> InNextSequenceIndex) override;


	/**
	 * Peek next video sample's timestamp
	 * @return true if value could be retrieved, false otherwise
	 */
	virtual bool PeekVideoSampleTime(FMediaTimeStamp & TimeStamp) override
	{
		TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe> Sample;
		if (!VideoSampleQueue.Peek(Sample))
		{
			return false;
		}
		TimeStamp = Sample->GetTime();
		return true;
	}

	virtual bool PeekVideoSampleTimeRanges(TArray<TRange<FMediaTimeStamp>>& TimeRange) override
	{
		VideoSampleQueue.GetSampleTimes(TimeRange);
		return TimeRange.Num() > 0;
	}
	virtual bool PeekAudioSampleTimeRanges(TArray<TRange<FMediaTimeStamp>>& TimeRange) override
	{
		AudioSampleQueue.GetSampleTimes(TimeRange);
		return TimeRange.Num() > 0;
	}

	virtual bool DiscardVideoSamples(const TRange<FMediaTimeStamp>& TimeRange, bool bReverse) override
	{
		return VideoSampleQueue.Discard(TimeRange, bReverse);
	}

	virtual bool DiscardAudioSamples(const TRange<FMediaTimeStamp>& TimeRange, bool bReverse) override
	{
		return AudioSampleQueue.Discard(TimeRange, bReverse);
	}

	virtual bool DiscardCaptionSamples(const TRange<FMediaTimeStamp>& TimeRange, bool bReverse) override
	{
		return CaptionSampleQueue.Discard(TimeRange, bReverse);
	}

	virtual bool DiscardSubtitleSamples(const TRange<FMediaTimeStamp>& TimeRange, bool bReverse) override
	{
		return SubtitleSampleQueue.Discard(TimeRange, bReverse);
	}

	virtual bool DiscardMetadataSamples(const TRange<FMediaTimeStamp>& TimeRange, bool bReverse) override
	{
		return MetadataSampleQueue.Discard(TimeRange, bReverse);
	}

	virtual uint32 GetNumDroppedVideoSamples(bool bInClearToZero) override 
	{ 
		return VideoSampleQueue.GetNumDroppedSamples(bInClearToZero); 
	}
	
	virtual uint32 GetNumDroppedAudioSamples(bool bInClearToZero) override 
	{ 
		return AudioSampleQueue.GetNumDroppedSamples(bInClearToZero); 
	}
	
	virtual uint32 GetNumDroppedSubtitleSamples(bool bInClearToZero) override 
	{ 
		return SubtitleSampleQueue.GetNumDroppedSamples(bInClearToZero); 
	}
	
	virtual uint32 GetNumDroppedCaptionSamples(bool bInClearToZero) override 
	{ 
		return CaptionSampleQueue.GetNumDroppedSamples(bInClearToZero); 
	}
	
	virtual uint32 GetNumDroppedMetadataSamples(bool bInClearToZero) override 
	{ 
		return MetadataSampleQueue.GetNumDroppedSamples(bInClearToZero); 
	}


	MEDIAUTILS_API virtual uint32 PurgeOutdatedVideoSamples(const FMediaTimeStamp & ReferenceTime, bool bReversed, FTimespan MaxAge) override;
	MEDIAUTILS_API virtual uint32 PurgeOutdatedSubtitleSamples(const FMediaTimeStamp & ReferenceTime, bool bReversed, FTimespan MaxAge) override;
	MEDIAUTILS_API virtual uint32 PurgeOutdatedCaptionSamples(const FMediaTimeStamp& ReferenceTime, bool bReversed, FTimespan MaxAge) override;
	MEDIAUTILS_API virtual uint32 PurgeOutdatedMetadataSamples(const FMediaTimeStamp& ReferenceTime, bool bReversed, FTimespan MaxAge) override;

	MEDIAUTILS_API virtual bool CanReceiveVideoSamples(uint32 Num) const override;
	MEDIAUTILS_API virtual bool CanReceiveAudioSamples(uint32 Num) const override;
	MEDIAUTILS_API virtual bool CanReceiveSubtitleSamples(uint32 Num) const override;
	MEDIAUTILS_API virtual bool CanReceiveCaptionSamples(uint32 Num) const override;
	MEDIAUTILS_API virtual bool CanReceiveMetadataSamples(uint32 Num) const override;

private:

	/** Audio sample queue. */
	FMediaAudioSampleQueue AudioSampleQueue;

	/** Caption sample queue. */
	FMediaOverlaySampleQueue CaptionSampleQueue;

	/** Metadata sample queue. */
	FMediaBinarySampleQueue MetadataSampleQueue;

	/** Subtitle sample queue. */
	FMediaOverlaySampleQueue SubtitleSampleQueue;

	/** Video sample queue. */
	FMediaTextureSampleQueue VideoSampleQueue;

	/** The next smallest expected sequence index if set. */
	TOptional<int32> NextMinSequenceIndex;
};
